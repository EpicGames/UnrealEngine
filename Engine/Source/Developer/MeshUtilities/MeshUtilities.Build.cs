// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MeshUtilities : ModuleRules
{
	public MeshUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
				"MaterialUtilities",

			}
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RawMesh",
				"RenderCore", // For FPackedNormal
				"SlateCore",
				"Slate",
				"MeshBoneReduction",
				"EditorFramework",
				"UnrealEd",
				"RHI",
				"HierarchicalLODUtilities",
				"Landscape",
				"LevelEditor",
				"PropertyEditor",
				
                "GraphColor",
                "MeshBuilderCommon",
                "MeshUtilitiesCommon",
                "MeshDescription",
				"StaticMeshDescription",
				"ToolMenus",
				"MeshUtilitiesEngine",
				"SkeletalMeshUtilitiesCommon",
				"ClothingSystemRuntimeCommon",
				"Persona",
			}
		);

        PublicIncludePathModuleNames.AddRange(
            new string[] {
                "MeshMergeUtilities"
            }
        );

        PrivateIncludePathModuleNames.AddRange(
          new string[] {
				"AnimationBlueprintEditor",
				"AnimationEditor",
                "MaterialBaking",
				"SkeletalMeshEditor",
          }
      );

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"AnimationBlueprintEditor",
				"AnimationEditor",
                "MeshMergeUtilities",
                "MaterialBaking",
				"SkeletalMeshEditor",
            }
        );

        AddEngineThirdPartyPrivateStaticDependencies(Target, "nvTriStrip");
        AddEngineThirdPartyPrivateStaticDependencies(Target, "ForsythTriOptimizer");
        AddEngineThirdPartyPrivateStaticDependencies(Target, "QuadricMeshReduction");
        AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "nvTessLib");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
            AddEngineThirdPartyPrivateStaticDependencies(Target, "DX9");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Embree");
	}
}
