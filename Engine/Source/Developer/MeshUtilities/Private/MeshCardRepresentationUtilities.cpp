// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshUtilities.h"
#include "MeshUtilitiesPrivate.h"
#include "Async/ParallelFor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "RawMesh.h"
#include "StaticMeshResources.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "DistanceFieldAtlas.h"
#include "MeshRepresentationCommon.h"
#include "MeshUtilitiesCommon.h"
#include "Containers/BinaryHeap.h"
#include <cmath>

static TAutoConsoleVariable<int32> CVarCardRepresentationParallelBuild(
	TEXT("r.MeshCardRepresentation.ParallelBuild"),
	1,
	TEXT("Whether to use task for mesh card building."),
	ECVF_Scalability);

namespace MeshCardGen
{
	int32 constexpr NumAxisAlignedDirections = 6;
	int32 constexpr MaxCardsPerMesh = 32;
};

class FGenerateCardMeshContext
{
public:
	const FString& MeshName;
	const FEmbreeScene& EmbreeScene;
	FCardRepresentationData& OutData;

	FGenerateCardMeshContext(const FString& InMeshName, const FEmbreeScene& InEmbreeScene, FCardRepresentationData& InOutData) :
		MeshName(InMeshName),
		EmbreeScene(InEmbreeScene),
		OutData(InOutData)
	{}
};

struct FIntBox
{
	FIntBox()
		: Min(INT32_MAX)
		, Max(-INT32_MAX)
	{}

	FIntBox(const FIntVector& InMin, const FIntVector& InMax)
		: Min(InMin)
		, Max(InMax)
	{}

	void Init()
	{
		Min = FIntVector(INT32_MAX);
		Max = FIntVector(-INT32_MAX);
	}

	void Add(const FIntVector& Point)
	{
		Min = FIntVector(FMath::Min(Min.X, Point.X), FMath::Min(Min.Y, Point.Y), FMath::Min(Min.Z, Point.Z));
		Max = FIntVector(FMath::Max(Max.X, Point.X), FMath::Max(Max.Y, Point.Y), FMath::Max(Max.Z, Point.Z));
	}

	FIntVector2 GetFaceXY() const
	{
		return FIntVector2(Max.X + 1 - Min.X, Max.Y + 1 - Min.Y);
	}

	int32 GetFaceArea() const
	{
		return (Max.X + 1 - Min.X) * (Max.Y + 1 - Min.Y);
	}

	bool Contains(const FIntBox& Other) const
	{
		if (Other.Min.X >= Min.X && Other.Max.X <= Max.X
			&& Other.Min.Y >= Min.Y && Other.Max.Y <= Max.Y
			&& Other.Min.Z >= Min.Z && Other.Max.Z <= Max.Z)
		{
			return true;
		}

		return false;
	}

	FIntVector GetAxisDistanceFromBox(const FIntBox& Box)
	{
		const FIntVector CenterDelta2 = (Max - Min) - (Box.Max - Box.Min);
		const FIntVector ExtentSum2 = (Max + Min) + (Box.Max + Box.Min);

		FIntVector AxisDistance;
		AxisDistance.X = FMath::Max(FMath::Abs(CenterDelta2.X) - ExtentSum2.X, 0) / 2;
		AxisDistance.Y = FMath::Max(FMath::Abs(CenterDelta2.Y) - ExtentSum2.Y, 0) / 2;
		AxisDistance.Z = FMath::Max(FMath::Abs(CenterDelta2.Z) - ExtentSum2.Z, 0) / 2;
		return AxisDistance;
	}

	FIntVector GetAxisDistanceFromPoint(const FIntVector& Point)
	{
		FIntVector AxisDistance;
		AxisDistance.X = FMath::Max(FMath::Max(Min.X - Point.X, Point.X - Max.X), 0);
		AxisDistance.Y = FMath::Max(FMath::Max(Min.Y - Point.Y, Point.Y - Max.Y), 0);
		AxisDistance.Z = FMath::Max(FMath::Max(Min.Z - Point.Z, Point.Z - Max.Z), 0);
		return AxisDistance;
	}

	FVector3f GetCenter() const
	{
		return FVector3f(Max + Min) * 0.5f;
	}

	FIntVector Min;
	FIntVector Max;
};

#if USE_EMBREE

typedef uint16 FSurfelIndex;
constexpr FSurfelIndex INVALID_SURFEL_INDEX = UINT16_MAX;

struct FSurfel
{
	FIntVector Coord;

	// Card's min near plane distance from this surfel
	int32 MinRayZ;

	// Percentage of rays which hit something in this cell
	float Coverage;

	// Coverage weighted by the visibility of this surfel from outside the mesh, decides how important is to cover this surfel
	float WeightedCoverage;
};

struct FSurfelScenePerDirection
{
	TArray<FSurfel> Surfels;
	FLumenCardBuildDebugData DebugData;

	void Init()
	{
		DebugData.Init();
		Surfels.Reset();
	}
};

struct FSurfelScene
{
	FSurfelScenePerDirection Directions[MeshCardGen::NumAxisAlignedDirections];
	int32 NumSurfels = 0;
};

struct FAxisAlignedDirectionBasis
{
	FMatrix44f LocalToWorldRotation;
	FVector3f LocalToWorldOffset;
	FIntVector VolumeSize;
	float VoxelSize;

	FVector3f TransformSurfel(FIntVector SurfelCoord) const
	{
		return LocalToWorldRotation.TransformPosition(FVector3f(SurfelCoord.X + 0.5f, SurfelCoord.Y + 0.5f, SurfelCoord.Z)) * VoxelSize + LocalToWorldOffset;
	}
};

struct FClusteringParams
{
	float VoxelSize = 0.0f;
	float MinClusterCoverage = 0.0f;
	float MinOuterClusterCoverage = 0.0f;
	float MinDensityPerCluster = 0.0f;
	int32 MaxLumenMeshCards = 0;
	bool bDebug = false;
	bool bSingleThreadedBuild = true;

	FAxisAlignedDirectionBasis ClusterBasis[MeshCardGen::NumAxisAlignedDirections];
};

FVector3f AxisAlignedDirectionIndexToNormal(int32 AxisAlignedDirectionIndex)
{
	const int32 AxisIndex = AxisAlignedDirectionIndex / 2;

	FVector3f Normal(0.0f, 0.0f, 0.0f);
	Normal[AxisIndex] = AxisAlignedDirectionIndex & 1 ? 1.0f : -1.0f;
	return Normal;
}

class FSurfelCluster
{
public:
	FIntBox Bounds;
	TArray<FSurfelIndex> SurfelIndices;

	int32 NearPlane = 0;
	float Coverage = 0.0f;

	// Coverage weighted by visibility
	float WeightedCoverage = 0.0f;

	// Best surfels to add to this cluster
	FSurfelIndex BestSurfelIndex = INVALID_SURFEL_INDEX;
	float BestSurfelDistance = FLT_MAX;

	void Init(int32 InNearPlane)
	{
		Bounds.Init();
		SurfelIndices.Reset();
		NearPlane = InNearPlane;
		Coverage = 0.0f;
		WeightedCoverage = 0.0f;
		BestSurfelIndex = INVALID_SURFEL_INDEX;
		BestSurfelDistance = FLT_MAX;
	}

	bool IsValid(const FClusteringParams& ClusteringParams) const
	{
		return WeightedCoverage >= (NearPlane == 0 ? ClusteringParams.MinOuterClusterCoverage : ClusteringParams.MinClusterCoverage)
			&& GetDensity() > ClusteringParams.MinDensityPerCluster;
	}

	float GetDensity() const
	{
		const float Density = Coverage / (float)Bounds.GetFaceArea();
		return Density;
	}

	void AddSurfel(const FSurfelScenePerDirection& SurfelScene, FSurfelIndex SurfelToAddIndex);
};

void FSurfelCluster::AddSurfel(const FSurfelScenePerDirection& SurfelScene, FSurfelIndex SurfelIndex)
{
	const FSurfel& Surfel = SurfelScene.Surfels[SurfelIndex];
	if (Surfel.Coord.Z >= NearPlane && Surfel.MinRayZ <= NearPlane)
	{
		SurfelIndices.Add(SurfelIndex);
		Bounds.Add(Surfel.Coord);
		Coverage += Surfel.Coverage;
		WeightedCoverage += Surfel.WeightedCoverage;

		// Check if all surfels are visible after add
		check(NearPlane <= Bounds.Min.Z);
	}
}

struct FSurfelSample
{
	FVector3f Position;
	FVector3f Normal;
	int32 MinRayZ;
	int32 CellZ;
};

struct FSurfelVisibility
{
	float Visibility;
	bool bValid;
};

// Trace rays over the hemisphere and discard surfels which mostly hit back faces
FSurfelVisibility ComputeSurfelVisibility(
	const FGenerateCardMeshContext& Context,
	const TArray<FSurfelSample>& SurfelSamples,
	uint32 SurfelSamplesOffset,
	uint32 SurfelSamplesNum,
	const TArray<FVector3f>& RayDirectionsOverHemisphere,
	FLumenCardBuildDebugData& DebugData)
{
	uint32 SurfelSampleIndex = 0;
	uint32 NumHits = 0;
	uint32 NumBackFaceHits = 0;
	const float SurfaceRayBias = 0.1f;
	float VisibilitySum = 0.0f;

	for (int32 RayIndex = 0; RayIndex < RayDirectionsOverHemisphere.Num(); ++RayIndex)
	{
		const FSurfelSample& SurfelSample = SurfelSamples[SurfelSampleIndex + SurfelSamplesOffset];
		const FMatrix44f SurfaceBasis = GetTangentBasisFrisvad(SurfelSample.Normal);
		const FVector3f RayOrigin = SurfelSample.Position;
		const FVector3f RayDirection = SurfaceBasis.TransformVector(RayDirectionsOverHemisphere[RayIndex]);

		FEmbreeRay EmbreeRay;
		EmbreeRay.ray.org_x = RayOrigin.X;
		EmbreeRay.ray.org_y = RayOrigin.Y;
		EmbreeRay.ray.org_z = RayOrigin.Z;
		EmbreeRay.ray.dir_x = RayDirection.X;
		EmbreeRay.ray.dir_y = RayDirection.Y;
		EmbreeRay.ray.dir_z = RayDirection.Z;
		EmbreeRay.ray.tnear = SurfaceRayBias;
		EmbreeRay.ray.tfar = FLT_MAX;
		EmbreeRay.ray.mask = 0xFFFFFFFF;

#if USE_EMBREE_MAJOR_VERSION >= 4
		FEmbreeRayQueryContext EmbreeContext;
		rtcInitRayQueryContext(&EmbreeContext);
		RTCIntersectArguments args;
		rtcInitIntersectArguments(&args);
		args.context = &EmbreeContext;
		rtcIntersect1(Context.EmbreeScene.Scene, &EmbreeRay, &args);
#else
		FEmbreeIntersectionContext EmbreeContext;
		rtcInitIntersectContext(&EmbreeContext);
		rtcIntersect1(Context.EmbreeScene.Scene, &EmbreeContext, &EmbreeRay);
#endif

		if (EmbreeRay.hit.geomID != RTC_INVALID_GEOMETRY_ID && EmbreeRay.hit.primID != RTC_INVALID_GEOMETRY_ID)
		{
			++NumHits;
			if (FVector::DotProduct((FVector)RayDirection, (FVector)EmbreeRay.GetHitNormal()) > 0.0f && !EmbreeContext.IsHitTwoSided())
			{
				++NumBackFaceHits;
			}
			else
			{
				VisibilitySum += 0.0f;
			}
		}
		else
		{
			VisibilitySum += 1.0f;
		}

#if 0
		FLumenCardBuildDebugData::FRay& SurfelRay = DebugData.SurfelRays.AddDefaulted_GetRef();
		SurfelRay.RayStart = RayOrigin;
		SurfelRay.RayEnd = RayOrigin + RayDirection * (EmbreeRay.ray.tfar < FLT_MAX ? EmbreeRay.ray.tfar : 200.0f);
		SurfelRay.bHit = EmbreeRay.ray.tfar < FLT_MAX;
#endif

		SurfelSampleIndex = (SurfelSampleIndex + 1) % SurfelSamplesNum;
	}

	const bool bInsideGeometry =
		NumHits > 0.8f * RayDirectionsOverHemisphere.Num()
		&& NumBackFaceHits > 0.2f * RayDirectionsOverHemisphere.Num();

	FSurfelVisibility SurfelVisibility;
	SurfelVisibility.Visibility = VisibilitySum / RayDirectionsOverHemisphere.Num();
	SurfelVisibility.bValid = !bInsideGeometry;
	return SurfelVisibility;
}

/**
 * Voxelize mesh by casting multiple rays per cell
 */
void GenerateSurfelsForDirection(
	const FGenerateCardMeshContext& Context,
	const FAxisAlignedDirectionBasis& ClusterBasis,	
	const TArray<FVector3f>& RayDirectionsOverHemisphere,
	const FClusteringParams& ClusteringParams,
	FSurfelScenePerDirection& SurfelScenePerDirection)
{
	const float NormalWeightTreshold = MeshCardRepresentation::GetNormalTreshold();
	const FVector3f RayDirection = ClusterBasis.LocalToWorldRotation.GetScaledAxis(EAxis::Type::Z);

	const uint32 NumSurfelSamples = 32;
	const uint32 MinSurfelSamples = 1;

	TArray<FSurfelSample> SurfelSamples;
	TArray<uint32> NumSurfelSamplesPerCell;
	TArray<uint32> SurfelSamplesOffsetPerCell;

	for (int32 CoordY = 0; CoordY < ClusterBasis.VolumeSize.Y; ++CoordY)
	{
		for (int32 CoordX = 0; CoordX < ClusterBasis.VolumeSize.X; ++CoordX)
		{
			SurfelSamples.Reset();
			NumSurfelSamplesPerCell.SetNum(ClusterBasis.VolumeSize.Z);
			SurfelSamplesOffsetPerCell.SetNum(ClusterBasis.VolumeSize.Z);
			for (int32 CoordZ = 0; CoordZ < ClusterBasis.VolumeSize.Z; ++CoordZ)
			{
				NumSurfelSamplesPerCell[CoordZ] = 0;
				SurfelSamplesOffsetPerCell[CoordZ] = 0;
			}

			// Trace multiple rays per cell and mark cells which need to spawn a surfel
			for (uint32 SampleIndex = 0; SampleIndex < NumSurfelSamples; ++SampleIndex)
			{
				FVector3f Jitter;
				Jitter.X = (SampleIndex + 0.5f) / NumSurfelSamples;
				Jitter.Y = (double)ReverseBits(SampleIndex) / (double)0x100000000LL;

				FVector3f RayOrigin = ClusterBasis.LocalToWorldRotation.TransformPosition(FVector3f(CoordX + Jitter.X, CoordY + Jitter.Y, 0.0f)) * ClusteringParams.VoxelSize + ClusterBasis.LocalToWorldOffset;

				// Need to pullback to make sure that ray will start outside of geometry, as voxels may be smaller than mesh 
				// due to voxel size rounding or they may start exactly at edge of geometry
				const float NearPlaneOffset = 2.0f * ClusteringParams.VoxelSize;
				RayOrigin -= RayDirection * NearPlaneOffset;

				// Cell index where any geometry was last found
				int32 LastHitCoordZ = -2;
				int32 SkipPrimId = RTC_INVALID_GEOMETRY_ID;
				float RayTNear = 0.0f;

				while (LastHitCoordZ + 1 < ClusterBasis.VolumeSize.Z)
				{
					FEmbreeRay EmbreeRay;
					EmbreeRay.ray.org_x = RayOrigin.X;
					EmbreeRay.ray.org_y = RayOrigin.Y;
					EmbreeRay.ray.org_z = RayOrigin.Z;
					EmbreeRay.ray.dir_x = RayDirection.X;
					EmbreeRay.ray.dir_y = RayDirection.Y;
					EmbreeRay.ray.dir_z = RayDirection.Z;
					EmbreeRay.ray.tnear = RayTNear;
					EmbreeRay.ray.tfar = FLT_MAX;
					EmbreeRay.ray.mask = 0xFFFFFFFF;

#if USE_EMBREE_MAJOR_VERSION >= 4
					FEmbreeRayQueryContext EmbreeContext;
					rtcInitRayQueryContext(&EmbreeContext);
					EmbreeContext.SkipPrimId = SkipPrimId;
					RTCIntersectArguments args;
					rtcInitIntersectArguments(&args);
					args.context = &EmbreeContext;
					rtcIntersect1(Context.EmbreeScene.Scene, &EmbreeRay, &args);
#else
					FEmbreeIntersectionContext EmbreeContext;
					rtcInitIntersectContext(&EmbreeContext);
					EmbreeContext.SkipPrimId = SkipPrimId;
					rtcIntersect1(Context.EmbreeScene.Scene, &EmbreeContext, &EmbreeRay);
#endif

					if (EmbreeRay.hit.geomID != RTC_INVALID_GEOMETRY_ID && EmbreeRay.hit.primID != RTC_INVALID_GEOMETRY_ID)
					{
						const int32 HitCoordZ = FMath::Clamp((EmbreeRay.ray.tfar - NearPlaneOffset) / ClusteringParams.VoxelSize, 0, ClusterBasis.VolumeSize.Z - 1);

						FVector SurfaceNormal = (FVector)EmbreeRay.GetHitNormal();
						float NdotD = FVector::DotProduct((FVector)-RayDirection, SurfaceNormal);

						// Handle two sided hits
						if (NdotD < 0.0f && EmbreeContext.IsHitTwoSided())
						{
							NdotD = -NdotD;
							SurfaceNormal = -SurfaceNormal;
						}

						const bool bPassProjectionTest = NdotD >= NormalWeightTreshold;
						if (bPassProjectionTest && HitCoordZ >= 0 && HitCoordZ > LastHitCoordZ + 1 && HitCoordZ < ClusterBasis.VolumeSize.Z)
						{
							FSurfelSample& SurfelSample = SurfelSamples.AddDefaulted_GetRef();
							SurfelSample.Position = RayOrigin + RayDirection * EmbreeRay.ray.tfar;
							SurfelSample.Normal = (FVector3f)SurfaceNormal;
							SurfelSample.CellZ = HitCoordZ;
							SurfelSample.MinRayZ = 0;

							if (LastHitCoordZ >= 0)
							{
								SurfelSample.MinRayZ = FMath::Max(SurfelSample.MinRayZ, LastHitCoordZ + 1);
							}
						}

						// Move ray to the next intersection
						LastHitCoordZ = HitCoordZ;
						// Sometimes EmbreeRay.ray.tnear was further than EmbreeRay.ray.tfar causing an infinite loop
						const float SafeTFar = FMath::Max(EmbreeRay.ray.tfar, EmbreeRay.ray.tnear);
						RayTNear = std::nextafter(FMath::Max(NearPlaneOffset + (LastHitCoordZ + 1) * ClusteringParams.VoxelSize, SafeTFar), std::numeric_limits<float>::infinity());
						SkipPrimId = EmbreeRay.hit.primID;
					}
					else
					{
						break;
					}
				}
			}

			// Sort surfel candidates and compact arrays
			{
				struct FSortByZ
				{
					FORCEINLINE bool operator()(const FSurfelSample& A, const FSurfelSample& B) const
					{
						if (A.CellZ != B.CellZ)
						{
							return A.CellZ < B.CellZ;
						}

						if (A.MinRayZ != B.MinRayZ)
						{
							return A.MinRayZ > B.MinRayZ;
						}

						if (A.Position.X != B.Position.X)
						{
							return A.Position.X < B.Position.X;
						}

						if (A.Position.Y != B.Position.Y)
						{
							return A.Position.Y < B.Position.Y;
						}

						return A.Position.Z < B.Position.Z;
					}
				};

				SurfelSamples.Sort(FSortByZ());

				for (int32 SampleIndex = 0; SampleIndex < SurfelSamples.Num(); ++SampleIndex)
				{
					const FSurfelSample& SurfelSample = SurfelSamples[SampleIndex];
					++NumSurfelSamplesPerCell[SurfelSample.CellZ];
				}

				for (int32 CoordZ = 1; CoordZ < ClusterBasis.VolumeSize.Z; ++CoordZ)
				{
					SurfelSamplesOffsetPerCell[CoordZ] = SurfelSamplesOffsetPerCell[CoordZ - 1] + NumSurfelSamplesPerCell[CoordZ - 1];
				}
			}

			// Convert surfel candidates into actual surfels
			for (int32 CoordZ = 0; CoordZ < ClusterBasis.VolumeSize.Z; ++CoordZ)
			{
				const int32 CellNumSurfelSamples = NumSurfelSamplesPerCell[CoordZ];
				const int32 CellSurfelSamplesOffset = SurfelSamplesOffsetPerCell[CoordZ];

				int32 SurfelSampleSpanBegin = 0;
				int32 SurfelSampleSpanSize = 0;

				bool bAnySurfelAdded = false;
				while (SurfelSampleSpanBegin + 1 < CellNumSurfelSamples)
				{
					// Find continuous spans of equal MinRayZ
					// Every such span will spawn one surfel
					SurfelSampleSpanSize = 0;
					for (int32 SampleIndex = SurfelSampleSpanBegin; SampleIndex < CellNumSurfelSamples; ++SampleIndex)
					{
						if (SurfelSamples[SampleIndex].MinRayZ == SurfelSamples[SurfelSampleSpanBegin].MinRayZ)
						{
							++SurfelSampleSpanSize;
						}
						else
						{
							break;
						}
					}

					if (SurfelSampleSpanSize >= MinSurfelSamples)
					{
						FSurfelVisibility SurfelVisibility = ComputeSurfelVisibility(
							Context,
							SurfelSamples,
							CellSurfelSamplesOffset + SurfelSampleSpanBegin,
							SurfelSampleSpanSize,
							RayDirectionsOverHemisphere,
							SurfelScenePerDirection.DebugData);

						const float Coverage = SurfelSampleSpanSize / float(NumSurfelSamples);

						if (SurfelVisibility.bValid)
						{
							const int32 MedianMinRayZ = SurfelSamples[CellSurfelSamplesOffset + SurfelSampleSpanBegin].MinRayZ;

							FSurfel& Surfel = SurfelScenePerDirection.Surfels.AddDefaulted_GetRef();
							Surfel.Coord = FIntVector(CoordX, CoordY, CoordZ);
							Surfel.MinRayZ = MedianMinRayZ;
							Surfel.Coverage = Coverage;
							Surfel.WeightedCoverage = Coverage * (SurfelVisibility.Visibility + 1.0f);
							check(Surfel.Coord.Z > Surfel.MinRayZ || Surfel.MinRayZ == 0);
						}

						if (ClusteringParams.bDebug)
						{
							FLumenCardBuildDebugData::FSurfel& DebugSurfel = SurfelScenePerDirection.DebugData.Surfels.AddDefaulted_GetRef();
							DebugSurfel.Position = ClusterBasis.TransformSurfel(FIntVector(CoordX, CoordY, CoordZ));
							DebugSurfel.Normal = -RayDirection;
							DebugSurfel.Coverage = Coverage;
							DebugSurfel.Visibility = SurfelVisibility.Visibility;
							DebugSurfel.SourceSurfelIndex = SurfelScenePerDirection.Surfels.Num() - 1;
							DebugSurfel.Type = SurfelVisibility.bValid ? FLumenCardBuildDebugData::ESurfelType::Valid : FLumenCardBuildDebugData::ESurfelType::Invalid;
							bAnySurfelAdded = true;
						}
					}

					SurfelSampleSpanBegin += SurfelSampleSpanSize;
				}

#define DEBUG_ADD_ALL_SURFELS 0
#if DEBUG_ADD_ALL_SURFELS
				if (ClusteringParams.bDebug && !bAnySurfelAdded)
				{
					FLumenCardBuildDebugData::FSurfel& DebugSurfel = SurfelScenePerDirection.DebugData.Surfels.AddDefaulted_GetRef();
					DebugSurfel.Position = ClusterBasis.TransformSurfel(FIntVector(CoordX, CoordY, CoordZ));
					DebugSurfel.Normal = -RayDirection;
					DebugSurfel.Coverage = 1.0f;
					DebugSurfel.Visibility = 1.0f;
					DebugSurfel.SourceSurfelIndex = SurfelScenePerDirection.Surfels.Num() - 1;
					DebugSurfel.Type = FLumenCardBuildDebugData::ESurfelType::Invalid;
				}
#endif
			}
		}
	}
}

void InitClusteringParams(FClusteringParams& ClusteringParams, const FBox& MeshCardsBounds, int32 MaxVoxels, int32 MaxLumenMeshCards)
{
	const float TargetVoxelSize = 10.0f;

	const FVector3f MeshCardsBoundsSize = 2.0f * (FVector3f)MeshCardsBounds.GetExtent();
	const float MaxMeshCardsBounds = MeshCardsBoundsSize.GetMax();

	// Target object space detail size
	const float MaxSizeInVoxels = FMath::Clamp(MaxMeshCardsBounds / TargetVoxelSize + 0.5f, 1, MaxVoxels);
	const float VoxelSize = FMath::Max(TargetVoxelSize, MaxMeshCardsBounds / MaxSizeInVoxels);

	FIntVector SizeInVoxels;
	SizeInVoxels.X = FMath::Clamp(FMath::RoundToFloat(MeshCardsBoundsSize.X / VoxelSize), 1, MaxVoxels);
	SizeInVoxels.Y = FMath::Clamp(FMath::RoundToFloat(MeshCardsBoundsSize.Y / VoxelSize), 1, MaxVoxels);
	SizeInVoxels.Z = FMath::Clamp(FMath::RoundToFloat(MeshCardsBoundsSize.Z / VoxelSize), 1, MaxVoxels);

	const FVector3f VoxelBoundsCenter = (FVector3f)MeshCardsBounds.GetCenter();
	const FVector3f VoxelBoundsExtent = FVector3f(SizeInVoxels) * VoxelSize * 0.5f;
	const FVector3f VoxelBoundsMin = VoxelBoundsCenter - VoxelBoundsExtent;
	const FVector3f VoxelBoundsMax = VoxelBoundsCenter + VoxelBoundsExtent;

	for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
	{
		FAxisAlignedDirectionBasis& ClusterBasis = ClusteringParams.ClusterBasis[AxisAlignedDirectionIndex];
		ClusterBasis.VoxelSize = VoxelSize;

		FVector3f XAxis = FVector3f(1.0f, 0.0f, 0.0f);
		FVector3f YAxis = FVector3f(0.0f, 1.0f, 0.0f);
		switch (AxisAlignedDirectionIndex / 2)
		{
		case 0:
			XAxis = FVector3f(0.0f, 1.0f, 0.0f);
			YAxis = FVector3f(0.0f, 0.0f, 1.0f);
			break;

		case 1:
			XAxis = FVector3f(1.0f, 0.0f, 0.0f);
			YAxis = FVector3f(0.0f, 0.0f, 1.0f);
			break;

		case 2:
			XAxis = FVector3f(1.0f, 0.0f, 0.0f);
			YAxis = FVector3f(0.0f, 1.0f, 0.0f);
			break;
		}

		FVector3f ZAxis = AxisAlignedDirectionIndexToNormal(AxisAlignedDirectionIndex);

		ClusterBasis.LocalToWorldRotation = FMatrix44f(XAxis, YAxis, -ZAxis, FVector3f::ZeroVector);

		ClusterBasis.LocalToWorldOffset = VoxelBoundsMin;
		if (AxisAlignedDirectionIndex & 1)
		{
			ClusterBasis.LocalToWorldOffset[AxisAlignedDirectionIndex / 2] = VoxelBoundsMax[AxisAlignedDirectionIndex / 2];
		}

		switch (AxisAlignedDirectionIndex / 2)
		{
		case 0:
			ClusterBasis.VolumeSize.X = SizeInVoxels.Y;
			ClusterBasis.VolumeSize.Y = SizeInVoxels.Z;
			ClusterBasis.VolumeSize.Z = SizeInVoxels.X;
			break;

		case 1:
			ClusterBasis.VolumeSize.X = SizeInVoxels.X;
			ClusterBasis.VolumeSize.Y = SizeInVoxels.Z;
			ClusterBasis.VolumeSize.Z = SizeInVoxels.Y;
			break;

		case 2:
			ClusterBasis.VolumeSize.X = SizeInVoxels.X;
			ClusterBasis.VolumeSize.Y = SizeInVoxels.Y;
			ClusterBasis.VolumeSize.Z = SizeInVoxels.Z;
			break;
		}
	}

	const float AverageFaceArea = 2.0f * (SizeInVoxels.X * SizeInVoxels.Y + SizeInVoxels.X * SizeInVoxels.Z + SizeInVoxels.Y * SizeInVoxels.Z) / 6.0f;

	ClusteringParams.VoxelSize = VoxelSize;
	ClusteringParams.MinDensityPerCluster = MeshCardRepresentation::GetMinDensity();
	ClusteringParams.MinDensityPerCluster = MeshCardRepresentation::GetMinDensity() / 3.0f;
	ClusteringParams.MinClusterCoverage = 15.0f;
	ClusteringParams.MinOuterClusterCoverage = FMath::Min(ClusteringParams.MinClusterCoverage, 0.5f * AverageFaceArea);
	ClusteringParams.MaxLumenMeshCards = MaxLumenMeshCards;
	ClusteringParams.bDebug = MeshCardRepresentation::IsDebugMode();
	ClusteringParams.bSingleThreadedBuild = CVarCardRepresentationParallelBuild.GetValueOnAnyThread() == 0;
}

void InitSurfelScene(
	const FGenerateCardMeshContext& Context,
	const FBox& MeshCardsBounds,
	int32 MaxLumenMeshCards,
	FSurfelScene& SurfelScene,
	FClusteringParams& ClusteringParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateSurfels);

	if (Context.EmbreeScene.NumTrianglesTotal == 0)
	{
		return;
	}

	// Generate random ray directions over a hemisphere
	constexpr uint32 NumRayDirectionsOverHemisphere = 32;
	TArray<FVector3f> RayDirectionsOverHemisphere;
	{
		FRandomStream RandomStream(0);
		MeshUtilities::GenerateStratifiedUniformHemisphereSamples(NumRayDirectionsOverHemisphere, RandomStream, RayDirectionsOverHemisphere);
	}

	const int32 DebugSurfelDirection = MeshCardRepresentation::GetDebugSurfelDirection();

	// Limit max number of surfels to prevent generation time from exploding, as dense two sided meshes can generate many more surfels than simple walls
	int32 TargetNumSufels = 10000;
	float MaxVoxels = 64;

	do
	{
		InitClusteringParams(ClusteringParams, MeshCardsBounds, MaxVoxels, MaxLumenMeshCards);

		ParallelFor(TEXT("InitSurfelScene.PF"), MeshCardGen::NumAxisAlignedDirections, 1,
			[&](int32 AxisAlignedDirectionIndex)
			{
				if (DebugSurfelDirection < 0 || DebugSurfelDirection == AxisAlignedDirectionIndex)
				{
					FSurfelScenePerDirection& SurfelScenePerDirection = SurfelScene.Directions[AxisAlignedDirectionIndex];
					SurfelScenePerDirection.Init();

					GenerateSurfelsForDirection(
						Context,
						ClusteringParams.ClusterBasis[AxisAlignedDirectionIndex],
						RayDirectionsOverHemisphere,
						ClusteringParams,
						SurfelScenePerDirection
					);
				}
			}, ClusteringParams.bSingleThreadedBuild ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

		SurfelScene.NumSurfels = 0;
		for (const FSurfelScenePerDirection& SurfelScenePerDirection : SurfelScene.Directions)
		{
			SurfelScene.NumSurfels += SurfelScenePerDirection.Surfels.Num();
		}

		MaxVoxels = MaxVoxels / 2;
	} while (SurfelScene.NumSurfels > TargetNumSufels && MaxVoxels > 1);

	if (ClusteringParams.bDebug)
	{
		for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
		{
			FLumenCardBuildDebugData& MergedDebugData = Context.OutData.MeshCardsBuildData.DebugData;
			FLumenCardBuildDebugData& DirectionDebugData = SurfelScene.Directions[AxisAlignedDirectionIndex].DebugData;

			const int32 SurfelOffset = MergedDebugData.Surfels.Num();

			MergedDebugData.Surfels.Append(DirectionDebugData.Surfels);
			MergedDebugData.SurfelRays.Append(DirectionDebugData.SurfelRays);

			for (FSurfelIndex SurfelIndex = SurfelOffset; SurfelIndex < MergedDebugData.Surfels.Num(); ++SurfelIndex)
			{
				MergedDebugData.Surfels[SurfelIndex].SourceSurfelIndex += SurfelOffset;
			}
		}
	}
}

struct FMeshCardsPerDirection
{
	TArray<FSurfelCluster> Clusters;
};

struct FMeshCards
{
	FMeshCardsPerDirection Directions[MeshCardGen::NumAxisAlignedDirections];
	float WeightedSurfaceCoverage = 0.0f;
	float SurfaceArea = 0.0f;
	int32 NumSurfels = 0;
	int32 NumClusters = 0;
};

void UpdateMeshCardsCoverage(const FSurfelScene& SurfelScene, const FClusteringParams& ClusteringParams, FMeshCards& MeshCards)
{
	MeshCards.WeightedSurfaceCoverage = 0.0f;
	MeshCards.SurfaceArea = 0.0f;
	MeshCards.NumSurfels = 0;
	MeshCards.NumClusters = 0;

	for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
	{
		TArray<FSurfelCluster>& Clusters = MeshCards.Directions[AxisAlignedDirectionIndex].Clusters;
		const TArray<FSurfel>& Surfels = SurfelScene.Directions[AxisAlignedDirectionIndex].Surfels;

		for (FSurfelCluster& Cluster : Clusters)
		{
			MeshCards.WeightedSurfaceCoverage += Cluster.WeightedCoverage;
			MeshCards.SurfaceArea += Cluster.Bounds.GetFaceArea();
		}

		MeshCards.NumSurfels += Surfels.Num();
		MeshCards.NumClusters += Clusters.Num();
	}
}

/**
 * Assign surfels to a single cluster
 */
void BuildCluster(
	int32 NearPlane,
	const FSurfelScenePerDirection& SurfelScene,
	TBitArray<>& SurfelAssignedToAnyCluster,
	FSurfelCluster& Cluster)
{
	Cluster.Init(NearPlane);

	for (int32 SurfelIndex = 0; SurfelIndex < SurfelScene.Surfels.Num(); ++SurfelIndex)
	{
		if (!SurfelAssignedToAnyCluster[SurfelIndex])
		{
			Cluster.AddSurfel(SurfelScene, SurfelIndex);
		}
	}
}

/**
 * Add cluster to the cluster list
 */
void CommitCluster(TArray<FSurfelCluster>& Clusters, TBitArray<>& SurfelAssignedToAnyCluster, FSurfelCluster const& Cluster)
{
	for (int32 SurfelIndex : Cluster.SurfelIndices)
	{
		SurfelAssignedToAnyCluster[SurfelIndex] = true;
	}
	Clusters.Add(Cluster);
}

/**
 * Sort clusters by importance and limit number of clusters based on the set target
 */
void LimitClusters(const FClusteringParams& ClusteringParams, const FSurfelScene& SurfelScene, FMeshCards& MeshCards)
{
	int32 NumClusters = 0;

	for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
	{
		TArray<FSurfelCluster>& Clusters = MeshCards.Directions[AxisAlignedDirectionIndex].Clusters;
		const TArray<FSurfel>& Surfels = SurfelScene.Directions[AxisAlignedDirectionIndex].Surfels;

		struct FSortByClusterWeightedCoverage
		{
			FORCEINLINE bool operator()(const FSurfelCluster& A, const FSurfelCluster& B) const
			{
				if (A.WeightedCoverage != B.WeightedCoverage)
				{
					return A.WeightedCoverage > B.WeightedCoverage;
				}

				const FVector3f CenterA = A.Bounds.GetCenter();
				const FVector3f CenterB = B.Bounds.GetCenter();
				
				if (CenterA.X != CenterB.X)
				{
					return CenterA.X < CenterB.X;
				}

				if (CenterA.Y != CenterB.Y)
				{
					return CenterA.Y < CenterB.Y;
				}

				return CenterA.Z < CenterB.Z;
			}
		};

		Clusters.Sort(FSortByClusterWeightedCoverage());
		NumClusters += Clusters.Num();
	}

	while (NumClusters > ClusteringParams.MaxLumenMeshCards)
	{
		float SmallestClusterWeightedCoverage = FLT_MAX;
		int32 SmallestClusterDirectionIndex = 0;

		for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
		{
			const TArray<FSurfelCluster>& Clusters = MeshCards.Directions[AxisAlignedDirectionIndex].Clusters;
			if (Clusters.Num() > 0)
			{
				const FSurfelCluster& Cluster = Clusters.Last();
				if (Cluster.WeightedCoverage < SmallestClusterWeightedCoverage)
				{
					SmallestClusterDirectionIndex = AxisAlignedDirectionIndex;
					SmallestClusterWeightedCoverage = Cluster.WeightedCoverage;
				}
			}
		}

		FMeshCardsPerDirection& MeshCardsPerDirection = MeshCards.Directions[SmallestClusterDirectionIndex];
		MeshCardsPerDirection.Clusters.Pop();
		--NumClusters;
	}
}

/**
 * Cover mesh using a set of clusters(cards)
 */
void BuildSurfelClusters(const FBox& MeshBounds, const FGenerateCardMeshContext& Context, const FSurfelScene& SurfelScene, const FClusteringParams& ClusteringParams, FMeshCards& MeshCards)
{
	TBitArray<> SurfelAssignedToAnyClusterArray[MeshCardGen::NumAxisAlignedDirections];
	for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
	{
		SurfelAssignedToAnyClusterArray[AxisAlignedDirectionIndex].Init(false, SurfelScene.Directions[AxisAlignedDirectionIndex].Surfels.Num());
	}

	ParallelFor(TEXT("BuildMeshCards.PF"), MeshCardGen::NumAxisAlignedDirections, 1,
		[&](int32 AxisAlignedDirectionIndex)
		{
			const FSurfelScenePerDirection& SurfelScenePerDirection = SurfelScene.Directions[AxisAlignedDirectionIndex];
			TArray<FSurfelCluster>& Clusters = MeshCards.Directions[AxisAlignedDirectionIndex].Clusters;
			TBitArray<>& SurfelAssignedToAnyCluster = SurfelAssignedToAnyClusterArray[AxisAlignedDirectionIndex];
			const FAxisAlignedDirectionBasis& ClusterBasis = ClusteringParams.ClusterBasis[AxisAlignedDirectionIndex];

			FSurfelCluster TempCluster;
			BuildCluster(/*NearPlane*/ 0, SurfelScenePerDirection, SurfelAssignedToAnyCluster, TempCluster);

			if (TempCluster.IsValid(ClusteringParams))
			{
				CommitCluster(Clusters, SurfelAssignedToAnyCluster, TempCluster);
			}

			// Assume that two sided is foliage and revert to a simpler box projection
			if (!Context.EmbreeScene.bMostlyTwoSided)
			{
				FSurfelCluster BestCluster;

				bool bCanAddCluster = true;
				while (bCanAddCluster)
				{
					BestCluster.Init(-1);

					for (int32 NearPlane = 1; NearPlane < ClusterBasis.VolumeSize.Z; ++NearPlane)
					{
						BuildCluster(NearPlane, SurfelScenePerDirection, SurfelAssignedToAnyCluster, TempCluster);

						if (TempCluster.IsValid(ClusteringParams) && TempCluster.WeightedCoverage > BestCluster.WeightedCoverage)
						{
							BestCluster = TempCluster;
						}
					}

					bCanAddCluster = BestCluster.IsValid(ClusteringParams);
					if (bCanAddCluster)
					{
						CommitCluster(Clusters, SurfelAssignedToAnyCluster, BestCluster);
					}
				}
			}

		}, ClusteringParams.bSingleThreadedBuild ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

	LimitClusters(ClusteringParams, SurfelScene, MeshCards);
}

void SerializeLOD(
	const FGenerateCardMeshContext& Context,
	const FClusteringParams& ClusteringParams,
	const FSurfelScene& SurfelScene,
	FMeshCards const& MeshCards,
	const FBox& MeshCardsBounds,
	FMeshCardsBuildData& MeshCardsBuildData)
{
	int32 SourceSurfelOffset = 0;

	for (int32 AxisAlignedDirectionIndex = 0; AxisAlignedDirectionIndex < MeshCardGen::NumAxisAlignedDirections; ++AxisAlignedDirectionIndex)
	{
		const FAxisAlignedDirectionBasis& ClusterBasis = ClusteringParams.ClusterBasis[AxisAlignedDirectionIndex];
		const FSurfelScenePerDirection& SurfelScenePerDirection = SurfelScene.Directions[AxisAlignedDirectionIndex];
		const TArray<FSurfel>& Surfels = SurfelScenePerDirection.Surfels;
		const TArray<FSurfelCluster>& Clusters = MeshCards.Directions[AxisAlignedDirectionIndex].Clusters;

		TBitArray<> DebugSurfelInCluster;
		TBitArray<> DebugSurfelInAnyCluster(false, Surfels.Num());

		const FBox3f LocalMeshCardsBounds = FBox3f(MeshCardsBounds.ShiftBy((FVector)-ClusterBasis.LocalToWorldOffset).TransformBy(FMatrix(ClusterBasis.LocalToWorldRotation.GetTransposed())));

		for (const FSurfelCluster& Cluster : Clusters)
		{
			// Set card to cover voxels, with a 0.5 voxel margin for the near/far plane
			FVector3f ClusterBoundsMin = (FVector3f(Cluster.Bounds.Min) - FVector3f(0.0f, 0.0f, 0.5f)) * ClusteringParams.VoxelSize;
			FVector3f ClusterBoundsMax = (FVector3f(Cluster.Bounds.Max) + FVector3f(1.0f, 1.0f, 1.5f)) * ClusteringParams.VoxelSize;

			// Clamp to mesh bounds
			// Leave small margin for Z as LOD/displacement may move it outside of bounds
			static float MarginZ = 10.0f;
			ClusterBoundsMin.X = FMath::Max(ClusterBoundsMin.X, LocalMeshCardsBounds.Min.X);
			ClusterBoundsMin.Y = FMath::Max(ClusterBoundsMin.Y, LocalMeshCardsBounds.Min.Y);
			ClusterBoundsMin.Z = FMath::Max(ClusterBoundsMin.Z, LocalMeshCardsBounds.Min.Z - MarginZ);
			ClusterBoundsMax.X = FMath::Min(ClusterBoundsMax.X, LocalMeshCardsBounds.Max.X);
			ClusterBoundsMax.Y = FMath::Min(ClusterBoundsMax.Y, LocalMeshCardsBounds.Max.Y);
			ClusterBoundsMax.Z = FMath::Min(ClusterBoundsMax.Z, LocalMeshCardsBounds.Max.Z + MarginZ);

			const FVector3f ClusterBoundsOrigin = (ClusterBoundsMax + ClusterBoundsMin) * 0.5f;
			const FVector3f ClusterBoundsExtent = (ClusterBoundsMax - ClusterBoundsMin) * 0.5f;
			const FVector3f MeshClusterBoundsOrigin = ClusterBasis.LocalToWorldRotation.TransformPosition(ClusterBoundsOrigin) + ClusterBasis.LocalToWorldOffset;

			FLumenCardBuildData BuiltData;
			BuiltData.OBB.Origin = MeshClusterBoundsOrigin;
			BuiltData.OBB.Extent = ClusterBoundsExtent;
			BuiltData.OBB.AxisX = ClusterBasis.LocalToWorldRotation.GetScaledAxis(EAxis::X);
			BuiltData.OBB.AxisY = ClusterBasis.LocalToWorldRotation.GetScaledAxis(EAxis::Y);
			BuiltData.OBB.AxisZ = -ClusterBasis.LocalToWorldRotation.GetScaledAxis(EAxis::Z);
			BuiltData.AxisAlignedDirectionIndex = AxisAlignedDirectionIndex;
			MeshCardsBuildData.CardBuildData.Add(BuiltData);

			if (ClusteringParams.bDebug)
			{
				DebugSurfelInCluster.Reset();
				DebugSurfelInCluster.Add(false, Surfels.Num());

				FLumenCardBuildDebugData::FSurfelCluster& DebugCluster = MeshCardsBuildData.DebugData.Clusters.AddDefaulted_GetRef();
				DebugCluster.Surfels.Reserve(DebugCluster.Surfels.Num() + Surfels.Num());

				for (FSurfelIndex SurfelIndex : Cluster.SurfelIndices)
				{
					FLumenCardBuildDebugData::FSurfel DebugSurfel;
					DebugSurfel.Position = ClusterBasis.TransformSurfel(Surfels[SurfelIndex].Coord);
					DebugSurfel.Normal = AxisAlignedDirectionIndexToNormal(AxisAlignedDirectionIndex);
					DebugSurfel.SourceSurfelIndex = SourceSurfelOffset + SurfelIndex;
					DebugSurfel.Type = FLumenCardBuildDebugData::ESurfelType::Cluster;
					DebugCluster.Surfels.Add(DebugSurfel);

					const int32 SurfelMinRayZ = Surfels[SurfelIndex].MinRayZ;
					if (SurfelMinRayZ > 0)
					{
						FIntVector MinRayZCoord = Surfels[SurfelIndex].Coord;
						MinRayZCoord.Z = SurfelMinRayZ;

						FLumenCardBuildDebugData::FRay DebugRay;
						DebugRay.RayStart = DebugSurfel.Position;
						DebugRay.RayEnd = ClusterBasis.TransformSurfel(MinRayZCoord);
						DebugRay.bHit = false;
						DebugCluster.Rays.Add(DebugRay);
					}

					DebugSurfelInAnyCluster[SurfelIndex] = true;
					DebugSurfelInCluster[SurfelIndex] = true;
				}

				for (FSurfelIndex SurfelIndex = 0; SurfelIndex < Surfels.Num(); ++SurfelIndex)
				{
					if (!DebugSurfelInCluster[SurfelIndex])
					{
						FLumenCardBuildDebugData::FSurfel DebugSurfel;
						DebugSurfel.Position = ClusterBasis.TransformSurfel(Surfels[SurfelIndex].Coord);
						DebugSurfel.Normal = AxisAlignedDirectionIndexToNormal(AxisAlignedDirectionIndex);
						DebugSurfel.SourceSurfelIndex = SourceSurfelOffset + SurfelIndex;
						DebugSurfel.Type = DebugSurfelInAnyCluster[SurfelIndex] ? FLumenCardBuildDebugData::ESurfelType::Used : FLumenCardBuildDebugData::ESurfelType::Idle;
						DebugCluster.Surfels.Add(DebugSurfel);
					}
				}
			}
		}

		SourceSurfelOffset += Surfels.Num();
	}

	if (ClusteringParams.bDebug)
	{
		UE_LOGF(LogMeshUtilities, Log, "CardGen Mesh:%ls Surfels:%d Clusters:%d WeightedSurfaceCoverage:%f ClusterArea:%f",
			*Context.MeshName,
			MeshCards.NumSurfels,
			MeshCards.NumClusters,
			MeshCards.WeightedSurfaceCoverage,
			MeshCards.SurfaceArea);
	}
}

void BuildMeshCards(const FBox& MeshBounds, const FGenerateCardMeshContext& Context, int32 MaxLumenMeshCards, FCardRepresentationData& OutData)
{
	// Make sure BBox isn't empty and we can generate card representation for it. This handles e.g. infinitely thin planes.
	const FVector MeshCardsBoundsCenter = MeshBounds.GetCenter();
	const FVector MeshCardsBoundsExtent = FVector::Max(MeshBounds.GetExtent() + 1.0f, FVector(1.0f));
	const FBox MeshCardsBounds(MeshCardsBoundsCenter - MeshCardsBoundsExtent, MeshCardsBoundsCenter + MeshCardsBoundsExtent);

	// Prepare a list of surfels for cluster fitting
	FSurfelScene SurfelScene;
	FClusteringParams ClusteringParams;
	InitSurfelScene(Context, MeshCardsBounds, MaxLumenMeshCards, SurfelScene, ClusteringParams);

	FMeshCards MeshCards;
	BuildSurfelClusters(MeshBounds, Context, SurfelScene, ClusteringParams, MeshCards);

	OutData.MeshCardsBuildData.Bounds = MeshCardsBounds;
	OutData.MeshCardsBuildData.bMostlyTwoSided = Context.EmbreeScene.bMostlyTwoSided;
	OutData.MeshCardsBuildData.CardBuildData.Reset();

	SerializeLOD(Context, ClusteringParams, SurfelScene, MeshCards, MeshCardsBounds, OutData.MeshCardsBuildData);

	OutData.MeshCardsBuildData.DebugData.NumSurfels = 0;
	for (const FSurfelScenePerDirection& SurfelScenePerDirection : SurfelScene.Directions)
	{
		 OutData.MeshCardsBuildData.DebugData.NumSurfels += SurfelScenePerDirection.Surfels.Num();
	}
}

#endif // #if USE_EMBREE

int64 FMeshUtilities::MemoryForGenerateMeshCardRepresentation(uint64_t IndexCount)
{
	// 1.5 was a derived from insights + LLM captures across a few different projects.
	return 1.5f * MeshRepresentation::MemoryEstimateForEmbreeScene(IndexCount); 
}

bool FMeshUtilities::GenerateCardRepresentationData(
	FString MeshName,
	const FMeshDataForDerivedDataTask& MeshData,
	const FDistanceFieldVolumeData* DistanceFieldVolumeData,
	int32 MaxLumenMeshCards,
	bool bGenerateAsIfTwoSided,
	FCardRepresentationData& OutData)
{
#if USE_EMBREE
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshUtilities::GenerateCardRepresentationData);

	if (MaxLumenMeshCards > 0)
	{
		const double StartTime = FPlatformTime::Seconds();

		// We include translucent triangles to get card representation for them in case translucent mesh tracing is used
		// in combination with hardware ray tracing for hit lighting.
		const bool bIncludeTranslucentTriangles = true; 

		FEmbreeScene EmbreeScene;
		MeshRepresentation::SetupEmbreeScene(MeshName, bGenerateAsIfTwoSided, EmbreeScene);

		if (!EmbreeScene.Scene)
		{
			return false;
		}

		if (!MeshRepresentation::AddMeshDataToEmbreeScene(EmbreeScene, MeshData, bIncludeTranslucentTriangles))
		{
			return false;
		}

		EmbreeScene.Commit();

		FGenerateCardMeshContext Context(MeshName, EmbreeScene, OutData);

		// Note: must operate on the SDF bounds when available, because SDF generation can expand the mesh's bounds
		const FBox3f BuildCardsBounds = DistanceFieldVolumeData && DistanceFieldVolumeData->LocalSpaceMeshBounds.IsValid ? DistanceFieldVolumeData->LocalSpaceMeshBounds : MeshData.Bounds.GetBox();
		BuildMeshCards(FBox(BuildCardsBounds), Context, MaxLumenMeshCards, OutData);

		MeshRepresentation::DeleteEmbreeScene(EmbreeScene);

		const float TimeElapsed = (float)(FPlatformTime::Seconds() - StartTime);
		if (TimeElapsed > 1.0f)
		{
			UE_LOGF(LogMeshUtilities, Log, "Finished mesh card build in %.1fs %ls tris:%d surfels:%d",
				TimeElapsed,
				*MeshName,
				EmbreeScene.NumTrianglesTotal,
				OutData.MeshCardsBuildData.DebugData.NumSurfels);
		}
	}

	return true;
#else
	UE_LOGF(LogMeshUtilities, Warning, "Platform did not set USE_EMBREE, GenerateCardRepresentationData failed.");
	return false;
#endif
}
