// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshUtilities.h"
#include "MeshUtilitiesPrivate.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "RawMesh.h"
#include "StaticMeshResources.h"
#include "DistanceFieldAtlas.h"
#include "MeshRepresentationCommon.h"
#include "Async/ParallelFor.h"
#include "ProfilingDebugging/CookStats.h"

#if ENABLE_COOK_STATS
namespace DistanceFieldUtilityCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("DistanceField.Usage"), TEXT(""));
	});
}
#endif

static TAutoConsoleVariable<int32> CVarDistanceFieldsEstimateRatio(
	TEXT("r.CookMemory.DistanceFieldsEstimateRatio"), 15,
	TEXT("Configurable ratio for the distance field memory estimate"),
	ECVF_ReadOnly
); 

#if USE_EMBREE


class FEmbreePointQueryContext : public RTCPointQueryContext
{
public:
	const FEmbreeScene* Scene = nullptr;
};

bool EmbreePointQueryFunction(RTCPointQueryFunctionArguments* args)
{
	const FEmbreePointQueryContext* Context = (const FEmbreePointQueryContext*)args->context;

	check(args->userPtr);
	float& ClosestDistanceSq = *(float*)(args->userPtr);

	int32 GeometryIndex = args->geomID;

	if (Context->instID[0] != RTC_INVALID_GEOMETRY_ID)
	{
		// when testing against a geometry instance use instID to index into Scene->Geometries
		GeometryIndex = Context->instID[0];
	}

	const FEmbreeGeometryAsset* GeometryAsset = Context->Scene->Geometries[GeometryIndex].Asset;
	const int32 NumTriangles = GeometryAsset->NumTriangles;

	const int32 TriangleIndex = args->primID;
	check(TriangleIndex < NumTriangles);

	const FVector3f* VertexBuffer = (const FVector3f*)GeometryAsset->VertexArray.GetData();
	const uint32* IndexBuffer = (const uint32*)GeometryAsset->IndexArray.GetData();

	const uint32 I0 = IndexBuffer[TriangleIndex * 3 + 0];
	const uint32 I1 = IndexBuffer[TriangleIndex * 3 + 1];
	const uint32 I2 = IndexBuffer[TriangleIndex * 3 + 2];

	FVector3f V0 = VertexBuffer[I0];
	FVector3f V1 = VertexBuffer[I1];
	FVector3f V2 = VertexBuffer[I2];

	if (Context->instID[0] != RTC_INVALID_GEOMETRY_ID)
	{
		// when testing against a geometry instance need to transform vertices to world space
		FMatrix44f* InstToWorld = (FMatrix44f*)Context->inst2world[0];

		V0 = InstToWorld->TransformPosition(V0);
		V1 = InstToWorld->TransformPosition(V1);
		V2 = InstToWorld->TransformPosition(V2);
	}

	const FVector3f QueryPosition(args->query->x, args->query->y, args->query->z);

	const FVector3f ClosestPoint = (FVector3f)FMath::ClosestPointOnTriangleToPoint((FVector)QueryPosition, (FVector)V0, (FVector)V1, (FVector)V2);
	const float QueryDistanceSq = (ClosestPoint - QueryPosition).SizeSquared();

	if (QueryDistanceSq < ClosestDistanceSq)
	{
		ClosestDistanceSq = QueryDistanceSq;

		bool bShrinkQuery = true;

		if (bShrinkQuery)
		{
			args->query->radius = FMath::Sqrt(ClosestDistanceSq);
			// Return true to indicate that the query radius has shrunk
			return true;
		}
	}

	// Return false to indicate that the query radius hasn't changed
	return false;
}

static int32 ComputeLinearVoxelIndex(FIntVector VoxelCoordinate, FIntVector VolumeDimensions)
{
	return (VoxelCoordinate.Z * VolumeDimensions.Y + VoxelCoordinate.Y) * VolumeDimensions.X + VoxelCoordinate.X;
}

class FSparseMeshDistanceFieldAsyncTask
{
public:
	FSparseMeshDistanceFieldAsyncTask(
		const FEmbreeScene& InEmbreeScene,
		const TArray<FVector3f>* InSampleDirections,
		float InLocalSpaceTraceDistance,
		FBox3f InVolumeBounds,
		float InLocalToVolumeScale,
		FVector2f InDistanceFieldToVolumeScaleBias,
		FInt32Vector InBrickCoordinate,
		FInt32Vector InIndirectionSize,
		bool bInUsePointQuery)
		:
		EmbreeScene(InEmbreeScene),
		SampleDirections(InSampleDirections),
		LocalSpaceTraceDistance(InLocalSpaceTraceDistance),
		VolumeBounds(InVolumeBounds),
		LocalToVolumeScale(InLocalToVolumeScale),
		DistanceFieldToVolumeScaleBias(InDistanceFieldToVolumeScaleBias),
		BrickCoordinate(InBrickCoordinate),
		IndirectionSize(InIndirectionSize),
		bUsePointQuery(bInUsePointQuery),
		BrickMaxDistance(MIN_uint8),
		BrickMinDistance(MAX_uint8)
	{}

	void DoWork();

	// Readonly inputs
	const FEmbreeScene& EmbreeScene;
	const TArray<FVector3f>* SampleDirections;
	float LocalSpaceTraceDistance;
	FBox VolumeBounds;
	float LocalToVolumeScale;
	FVector2D DistanceFieldToVolumeScaleBias;
	FIntVector BrickCoordinate;
	FIntVector IndirectionSize;
	bool bUsePointQuery;

	// Output
	uint8 BrickMaxDistance;
	uint8 BrickMinDistance;
	TArray<uint8> DistanceFieldVolume;
};

int32 DebugX = 0;
int32 DebugY = 0;
int32 DebugZ = 0;

void FSparseMeshDistanceFieldAsyncTask::DoWork()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSparseMeshDistanceFieldAsyncTask::DoWork);
	COOK_STAT(auto Timer = DistanceFieldUtilityCookStats::UsageStats.TimeSyncWork());
	COOK_STAT(Timer.TrackCyclesOnly());

	const FVector IndirectionVoxelSize = VolumeBounds.GetSize() / FVector(IndirectionSize);
	const FVector DistanceFieldVoxelSize = IndirectionVoxelSize / FVector(DistanceField::UniqueDataBrickSize);
	const FVector BrickMinPosition = VolumeBounds.Min + FVector(BrickCoordinate) * IndirectionVoxelSize;

	DistanceFieldVolume.Empty(DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize);
	DistanceFieldVolume.AddZeroed(DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize);

	for (int32 ZIndex = 0; ZIndex < DistanceField::BrickSize; ZIndex++)
	{
		for (int32 YIndex = 0; YIndex < DistanceField::BrickSize; YIndex++)
		{
			for (int32 XIndex = 0; XIndex < DistanceField::BrickSize; XIndex++)
			{
				if (XIndex == DebugX && YIndex == DebugY && ZIndex == DebugZ)
				{
					int32 DebugBreak = 0;
				}

				const FVector VoxelPosition = FVector(XIndex, YIndex, ZIndex) * DistanceFieldVoxelSize + BrickMinPosition;
				const int32 Index = (ZIndex * DistanceField::BrickSize * DistanceField::BrickSize + YIndex * DistanceField::BrickSize + XIndex);

				float MinLocalSpaceDistance = LocalSpaceTraceDistance;

				bool bTraceRays = true;

				if (bUsePointQuery)
				{
					RTCPointQuery PointQuery;
					PointQuery.x = VoxelPosition.X;
					PointQuery.y = VoxelPosition.Y;
					PointQuery.z = VoxelPosition.Z;
					PointQuery.time = 0;
					PointQuery.radius = LocalSpaceTraceDistance;

					FEmbreePointQueryContext QueryContext;
					rtcInitPointQueryContext(&QueryContext);
					QueryContext.Scene = &EmbreeScene;
					float ClosestUnsignedDistanceSq = (LocalSpaceTraceDistance * 2.0f) * (LocalSpaceTraceDistance * 2.0f);
					rtcPointQuery(EmbreeScene.Scene, &PointQuery, &QueryContext, EmbreePointQueryFunction, &ClosestUnsignedDistanceSq);

					const float ClosestDistance = FMath::Sqrt(ClosestUnsignedDistanceSq);
					bTraceRays = ClosestDistance <= LocalSpaceTraceDistance;
					MinLocalSpaceDistance = FMath::Min(MinLocalSpaceDistance, ClosestDistance);
				}
				
				if (bTraceRays)
				{
					int32 Hit = 0;
					int32 HitBack = 0;

					for (int32 SampleIndex = 0; SampleIndex < SampleDirections->Num(); SampleIndex++)
					{
						const FVector UnitRayDirection = (FVector)(*SampleDirections)[SampleIndex];
						const float PullbackEpsilon = 1.e-4f;
						// Pull back the starting position slightly to make sure we hit a triangle that VoxelPosition is exactly on.  
						// This happens a lot with boxes, since we trace from voxel corners.
						const FVector StartPosition = VoxelPosition - PullbackEpsilon * LocalSpaceTraceDistance * UnitRayDirection;
						const FVector EndPosition = VoxelPosition + UnitRayDirection * LocalSpaceTraceDistance;

						if (FMath::LineBoxIntersection(VolumeBounds, VoxelPosition, EndPosition, UnitRayDirection))
						{
							FEmbreeRay EmbreeRay;

							FVector RayDirection = EndPosition - VoxelPosition;
							EmbreeRay.ray.org_x = StartPosition.X;
							EmbreeRay.ray.org_y = StartPosition.Y;
							EmbreeRay.ray.org_z = StartPosition.Z;
							EmbreeRay.ray.dir_x = RayDirection.X;
							EmbreeRay.ray.dir_y = RayDirection.Y;
							EmbreeRay.ray.dir_z = RayDirection.Z;
							EmbreeRay.ray.tnear = 0;
							EmbreeRay.ray.tfar = 1.0f;
							EmbreeRay.ray.mask = 0xFFFFFFFF;

#if USE_EMBREE_MAJOR_VERSION >= 4
							FEmbreeRayQueryContext EmbreeContext;
							rtcInitRayQueryContext(&EmbreeContext);
							RTCIntersectArguments args;
							rtcInitIntersectArguments(&args);
							args.context = &EmbreeContext;
							rtcIntersect1(EmbreeScene.Scene, &EmbreeRay, &args);
#else
							FEmbreeIntersectionContext EmbreeContext;
							rtcInitIntersectContext(&EmbreeContext);
							rtcIntersect1(EmbreeScene.Scene, &EmbreeContext, &EmbreeRay);
#endif

							if (EmbreeRay.hit.geomID != RTC_INVALID_GEOMETRY_ID && EmbreeRay.hit.primID != RTC_INVALID_GEOMETRY_ID)
							{
								check(EmbreeContext.ElementIndex != -1);
								Hit++;

								const FVector HitNormal = (FVector)EmbreeRay.GetHitNormal();

								if (FVector::DotProduct(UnitRayDirection, HitNormal) > 0 && !EmbreeContext.IsHitTwoSided())
								{
									HitBack++;
								}

								if (!bUsePointQuery)
								{
									const float CurrentDistance = EmbreeRay.ray.tfar * LocalSpaceTraceDistance;

									if (CurrentDistance < MinLocalSpaceDistance)
									{
										MinLocalSpaceDistance = CurrentDistance;
									}
								}
							}
						}
					}

					// Consider this voxel 'inside' an object if we hit a significant number of backfaces
					if (Hit > 0 && HitBack > .25f * SampleDirections->Num())
					{
						MinLocalSpaceDistance *= -1;
					}
				}

				// Transform to the tracing shader's Volume space
				const float VolumeSpaceDistance = MinLocalSpaceDistance * LocalToVolumeScale;
				// Transform to the Distance Field texture's space
				const float RescaledDistance = (VolumeSpaceDistance - DistanceFieldToVolumeScaleBias.Y) / DistanceFieldToVolumeScaleBias.X;
				check(DistanceField::DistanceFieldFormat == PF_G8);
				const uint8 QuantizedDistance = FMath::Clamp<int32>(FMath::FloorToInt(RescaledDistance * 255.0f + .5f), 0, 255);
				DistanceFieldVolume[Index] = QuantizedDistance;
				BrickMaxDistance = FMath::Max(BrickMaxDistance, QuantizedDistance);
				BrickMinDistance = FMath::Min(BrickMinDistance, QuantizedDistance);
			}
		}
	}
}

int64_t FMeshUtilities::MemoryForGenerateSignedDistanceFieldVolumeData(
	const FMeshDataForDerivedDataTask& MeshData,
	float DistanceFieldResolutionScale
)
{
	// ------------------
	// Copied from GenerateSignedDistanceFieldVolumeData
	// ------------------
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DistanceFields.MaxPerMeshResolution"));
	const int32 PerMeshMax = CVar->GetValueOnAnyThread();

	// Meshes with explicit artist-specified scale can go higher
	const int32 MaxNumBlocksOneDim = FMath::Min<int32>(FMath::DivideAndRoundNearest(DistanceFieldResolutionScale <= 1 ? PerMeshMax / 2 : PerMeshMax, DistanceField::UniqueDataBrickSize), DistanceField::MaxIndirectionDimension - 1);

	static const auto CVarDensity = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DistanceFields.DefaultVoxelDensity"));
	const float VoxelDensity = CVarDensity->GetValueOnAnyThread();

	const float NumVoxelsPerLocalSpaceUnit = VoxelDensity * DistanceFieldResolutionScale;

	FBox3f LocalSpaceMeshBounds(MeshData.Bounds.GetBox());

	// Make sure the mesh bounding box has positive extents to handle planes
	{
		FVector3f MeshBoundsCenter = LocalSpaceMeshBounds.GetCenter();
		FVector3f MeshBoundsExtent = FVector3f::Max(LocalSpaceMeshBounds.GetExtent(), FVector3f::OneVector);
		LocalSpaceMeshBounds.Min = MeshBoundsCenter - MeshBoundsExtent;
		LocalSpaceMeshBounds.Max = MeshBoundsCenter + MeshBoundsExtent;
	}
	const FVector3f DesiredDimensions = FVector3f(LocalSpaceMeshBounds.GetSize() * NumVoxelsPerLocalSpaceUnit / (float)DistanceField::UniqueDataBrickSize);

	const FInt32Vector Mip0IndirectionDimensions = FInt32Vector(
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.X), 1, MaxNumBlocksOneDim),
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Y), 1, MaxNumBlocksOneDim),
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Z), 1, MaxNumBlocksOneDim));

	// ------------------

	const int64 TotalBrickCount = (Mip0IndirectionDimensions.X * DistanceField::UniqueDataBrickSize 
		* Mip0IndirectionDimensions.Y * DistanceField::UniqueDataBrickSize
		* Mip0IndirectionDimensions.Z * DistanceField::UniqueDataBrickSize);

	// Base and Ratio have been taken by calculating at a granular level the memory used and correlating 
	// it with the TotalBrickCount. An estimate was used in place of granular calculations for simplicity.
	const int32 Base = 1000;
	const int32 Ratio = CVarDistanceFieldsEstimateRatio.GetValueOnAnyThread();

	const int64 MeshUtilTotal = Base + TotalBrickCount * Ratio;

	// ------------------

	const uint32 NumIndices = MeshData.SourceMeshData ? MeshData.SourceMeshData->GetNumIndices() : MeshData.LODModel->IndexBuffer.GetNumIndices();
	const int64 EmbreeUsage = MeshRepresentation::MemoryEstimateForEmbreeScene(NumIndices);

	// ------------------

	const int64 Total = MeshUtilTotal + EmbreeUsage;
	return Total;
}

static void BuildSignedDistanceField(
	FString MeshName,
	const FEmbreeScene& EmbreeScene,
	const FBoxSphereBounds3f& Bounds,
	float DistanceFieldResolutionScale,
	FDistanceFieldVolumeData& OutData)
{
	// Whether to use an Embree Point Query to compute the closest unsigned distance.  Rays will only be traced to determine backfaces visible for sign.
	const bool bUsePointQuery = true;

	TArray<FVector3f> SampleDirections;
	{
		const int32 NumVoxelDistanceSamples = bUsePointQuery ? 49 : 576;
		FRandomStream RandomStream(0);
		MeshUtilities::GenerateStratifiedUniformHemisphereSamples(NumVoxelDistanceSamples, RandomStream, SampleDirections);
		TArray<FVector3f> OtherHemisphereSamples;
		MeshUtilities::GenerateStratifiedUniformHemisphereSamples(NumVoxelDistanceSamples, RandomStream, OtherHemisphereSamples);

		for (int32 i = 0; i < OtherHemisphereSamples.Num(); i++)
		{
			FVector3f Sample = OtherHemisphereSamples[i];
			Sample.Z *= -1.0f;
			SampleDirections.Add(Sample);
		}
	}

	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DistanceFields.MaxPerMeshResolution"));
	const int32 PerMeshMax = CVar->GetValueOnAnyThread();

	// Meshes with explicit artist-specified scale can go higher
	const int32 MaxNumBlocksOneDim = FMath::Min<int32>(FMath::DivideAndRoundNearest(DistanceFieldResolutionScale <= 1 ? PerMeshMax / 2 : PerMeshMax, DistanceField::UniqueDataBrickSize), DistanceField::MaxIndirectionDimension - 1);

	static const auto CVarDensity = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.DistanceFields.DefaultVoxelDensity"));
	const float VoxelDensity = CVarDensity->GetValueOnAnyThread();

	const float NumVoxelsPerLocalSpaceUnit = VoxelDensity * DistanceFieldResolutionScale;
	FBox3f LocalSpaceMeshBounds(Bounds.GetBox());

	// Make sure the mesh bounding box has positive extents to handle planes
	{
		FVector3f MeshBoundsCenter = LocalSpaceMeshBounds.GetCenter();
		FVector3f MeshBoundsExtent = FVector3f::Max(LocalSpaceMeshBounds.GetExtent(), FVector3f::OneVector);
		LocalSpaceMeshBounds.Min = MeshBoundsCenter - MeshBoundsExtent;
		LocalSpaceMeshBounds.Max = MeshBoundsCenter + MeshBoundsExtent;
	}

	// We sample on voxel corners and use central differencing for gradients, so a box mesh using two-sided materials whose vertices lie on LocalSpaceMeshBounds produces a zero gradient on intersection
	// Expand the mesh bounds by a fraction of a voxel to allow room for a pullback on the hit location for computing the gradient.
	// Only expand for two sided meshes as this adds significant Mesh SDF tracing cost
	if (EmbreeScene.bMostlyTwoSided)
	{
		const FVector3f DesiredDimensions = LocalSpaceMeshBounds.GetSize() * FVector3f(NumVoxelsPerLocalSpaceUnit / (float)DistanceField::UniqueDataBrickSize);
		const FInt32Vector Mip0IndirectionDimensions = FInt32Vector(
			FMath::Clamp(FMath::RoundToInt(DesiredDimensions.X), 1, MaxNumBlocksOneDim),
			FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Y), 1, MaxNumBlocksOneDim),
			FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Z), 1, MaxNumBlocksOneDim));

		const float CentralDifferencingExpandInVoxels = .25f;
		const FVector3f TexelObjectSpaceSize = LocalSpaceMeshBounds.GetSize() / FVector3f(Mip0IndirectionDimensions * DistanceField::UniqueDataBrickSize - FInt32Vector(2 * CentralDifferencingExpandInVoxels));
		LocalSpaceMeshBounds = LocalSpaceMeshBounds.ExpandBy(TexelObjectSpaceSize);
	}

	// The tracing shader uses a Volume space that is normalized by the maximum extent, to keep Volume space within [-1, 1], we must match that behavior when encoding
	const float LocalToVolumeScale = 1.0f / LocalSpaceMeshBounds.GetExtent().GetMax();

	const FVector3f DesiredDimensions = FVector3f(LocalSpaceMeshBounds.GetSize() * NumVoxelsPerLocalSpaceUnit / (float)DistanceField::UniqueDataBrickSize);
	const FInt32Vector Mip0IndirectionDimensions = FInt32Vector(
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.X), 1, MaxNumBlocksOneDim),
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Y), 1, MaxNumBlocksOneDim),
		FMath::Clamp(FMath::RoundToInt(DesiredDimensions.Z), 1, MaxNumBlocksOneDim));

	TArray<uint8> StreamableMipData;

	for (int32 MipIndex = 0; MipIndex < DistanceField::NumMips; MipIndex++)
	{
		const FInt32Vector IndirectionDimensions = FInt32Vector(
			FMath::DivideAndRoundUp(Mip0IndirectionDimensions.X, 1 << MipIndex),
			FMath::DivideAndRoundUp(Mip0IndirectionDimensions.Y, 1 << MipIndex),
			FMath::DivideAndRoundUp(Mip0IndirectionDimensions.Z, 1 << MipIndex));

		// Expand to guarantee one voxel border for gradient reconstruction using bilinear filtering
		const FVector3f TexelObjectSpaceSize = LocalSpaceMeshBounds.GetSize() / FVector3f(IndirectionDimensions * DistanceField::UniqueDataBrickSize - FIntVector(2 * DistanceField::MeshDistanceFieldObjectBorder));
		const FBox3f DistanceFieldVolumeBounds = LocalSpaceMeshBounds.ExpandBy(TexelObjectSpaceSize);

		const FVector3f IndirectionVoxelSize = DistanceFieldVolumeBounds.GetSize() / FVector3f(IndirectionDimensions);
		const float IndirectionVoxelRadius = IndirectionVoxelSize.Size();

		const FVector3f VolumeSpaceDistanceFieldVoxelSize = IndirectionVoxelSize * LocalToVolumeScale / FVector3f(DistanceField::UniqueDataBrickSize);
		const float MaxDistanceForEncoding = VolumeSpaceDistanceFieldVoxelSize.Size() * DistanceField::BandSizeInVoxels;
		const float LocalSpaceTraceDistance = MaxDistanceForEncoding / LocalToVolumeScale;
		const FVector2f DistanceFieldToVolumeScaleBias(2.0f * MaxDistanceForEncoding, -MaxDistanceForEncoding);

		TArray<FSparseMeshDistanceFieldAsyncTask> AsyncTasks;
		AsyncTasks.Reserve(IndirectionDimensions.X * IndirectionDimensions.Y * IndirectionDimensions.Z);

		for (int32 ZIndex = 0; ZIndex < IndirectionDimensions.Z; ZIndex++)
		{
			for (int32 YIndex = 0; YIndex < IndirectionDimensions.Y; YIndex++)
			{
				for (int32 XIndex = 0; XIndex < IndirectionDimensions.X; XIndex++)
				{
					AsyncTasks.Emplace(
						EmbreeScene,
						&SampleDirections,
						LocalSpaceTraceDistance,
						DistanceFieldVolumeBounds,
						LocalToVolumeScale,
						DistanceFieldToVolumeScaleBias,
						FInt32Vector(XIndex, YIndex, ZIndex),
						IndirectionDimensions,
						bUsePointQuery);
				}
			}
		}

		static bool bMultiThreaded = true;

		if (bMultiThreaded)
		{
			EParallelForFlags Flags = EParallelForFlags::BackgroundPriority | EParallelForFlags::Unbalanced;
			COOK_STAT(auto AsyncTimer = DistanceFieldUtilityCookStats::UsageStats.TimeAsyncWait());
			COOK_STAT(AsyncTimer.TrackCyclesOnly());

			ParallelForTemplate(
				TEXT("GenerateSignedDistanceFieldVolumeData.PF"),
				AsyncTasks.Num(), 1, [&AsyncTasks](int32 TaskIndex)
				{
					AsyncTasks[TaskIndex].DoWork();
				}, Flags);
		}
		else
		{
			for (FSparseMeshDistanceFieldAsyncTask& AsyncTask : AsyncTasks)
			{
				AsyncTask.DoWork();
			}
		}

		FSparseDistanceFieldMip& OutMip = OutData.Mips[MipIndex];
		TArray<uint32> IndirectionTable;
		IndirectionTable.Empty(IndirectionDimensions.X * IndirectionDimensions.Y * IndirectionDimensions.Z);
		IndirectionTable.AddUninitialized(IndirectionDimensions.X * IndirectionDimensions.Y * IndirectionDimensions.Z);

		for (int32 i = 0; i < IndirectionTable.Num(); i++)
		{
			IndirectionTable[i] = DistanceField::InvalidBrickIndex;
		}

		TArray<FSparseMeshDistanceFieldAsyncTask*> ValidBricks;
		ValidBricks.Empty(AsyncTasks.Num());

		for (int32 TaskIndex = 0; TaskIndex < AsyncTasks.Num(); TaskIndex++)
		{
			if (AsyncTasks[TaskIndex].BrickMinDistance < MAX_uint8 && AsyncTasks[TaskIndex].BrickMaxDistance > MIN_uint8)
			{
				ValidBricks.Add(&AsyncTasks[TaskIndex]);
			}
		}

		const uint32 NumBricks = ValidBricks.Num();

		const uint32 BrickSizeBytes = DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize * GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes;

		TArray<uint8> DistanceFieldBrickData;
		DistanceFieldBrickData.Empty(BrickSizeBytes * NumBricks);
		DistanceFieldBrickData.AddUninitialized(BrickSizeBytes * NumBricks);

		for (int32 BrickIndex = 0; BrickIndex < ValidBricks.Num(); BrickIndex++)
		{
			const FSparseMeshDistanceFieldAsyncTask& Brick = *ValidBricks[BrickIndex];
			const int32 IndirectionIndex = ComputeLinearVoxelIndex(Brick.BrickCoordinate, IndirectionDimensions);
			IndirectionTable[IndirectionIndex] = BrickIndex;

			check(BrickSizeBytes == Brick.DistanceFieldVolume.Num() * Brick.DistanceFieldVolume.GetTypeSize());
			FPlatformMemory::Memcpy(&DistanceFieldBrickData[BrickIndex * BrickSizeBytes], Brick.DistanceFieldVolume.GetData(), Brick.DistanceFieldVolume.Num() * Brick.DistanceFieldVolume.GetTypeSize());
		}

		const int32 IndirectionTableBytes = IndirectionTable.Num() * IndirectionTable.GetTypeSize();
		const int32 MipDataBytes = IndirectionTableBytes + DistanceFieldBrickData.Num();

		if (MipIndex == DistanceField::NumMips - 1)
		{
			OutData.AlwaysLoadedMip.Empty(MipDataBytes);
			OutData.AlwaysLoadedMip.AddUninitialized(MipDataBytes);

			FPlatformMemory::Memcpy(&OutData.AlwaysLoadedMip[0], IndirectionTable.GetData(), IndirectionTableBytes);

			if (DistanceFieldBrickData.Num() > 0)
			{
				FPlatformMemory::Memcpy(&OutData.AlwaysLoadedMip[IndirectionTableBytes], DistanceFieldBrickData.GetData(), DistanceFieldBrickData.Num());
			}
		}
		else
		{
			OutMip.BulkOffset = StreamableMipData.Num();
			StreamableMipData.AddUninitialized(MipDataBytes);
			OutMip.BulkSize = StreamableMipData.Num() - OutMip.BulkOffset;
			checkf(OutMip.BulkSize > 0, TEXT("BulkSize was 0 for %s with %ux%ux%u indirection"), *MeshName, IndirectionDimensions.X, IndirectionDimensions.Y, IndirectionDimensions.Z);

			FPlatformMemory::Memcpy(&StreamableMipData[OutMip.BulkOffset], IndirectionTable.GetData(), IndirectionTableBytes);

			if (DistanceFieldBrickData.Num() > 0)
			{
				FPlatformMemory::Memcpy(&StreamableMipData[OutMip.BulkOffset + IndirectionTableBytes], DistanceFieldBrickData.GetData(), DistanceFieldBrickData.Num());
			}
		}

		OutMip.IndirectionDimensions = IndirectionDimensions;
		OutMip.DistanceFieldToVolumeScaleBias = DistanceFieldToVolumeScaleBias;
		OutMip.NumDistanceFieldBricks = NumBricks;

		// Account for the border voxels we added
		const FVector3f VirtualUVMin = FVector3f(DistanceField::MeshDistanceFieldObjectBorder) / FVector3f(IndirectionDimensions * DistanceField::UniqueDataBrickSize);
		const FVector3f VirtualUVSize = FVector3f(IndirectionDimensions * DistanceField::UniqueDataBrickSize - FInt32Vector(2 * DistanceField::MeshDistanceFieldObjectBorder)) / FVector3f(IndirectionDimensions * DistanceField::UniqueDataBrickSize);

		const FVector3f VolumePositionExtent = LocalSpaceMeshBounds.GetExtent() * LocalToVolumeScale;

		// [-VolumePositionExtent, VolumePositionExtent] -> [VirtualUVMin, VirtualUVMin + VirtualUVSize]
		OutMip.VolumeToVirtualUVScale = VirtualUVSize / (2 * VolumePositionExtent);
		OutMip.VolumeToVirtualUVAdd = VolumePositionExtent * OutMip.VolumeToVirtualUVScale + VirtualUVMin;
	}

	OutData.bMostlyTwoSided = EmbreeScene.bMostlyTwoSided;
	OutData.LocalSpaceMeshBounds = LocalSpaceMeshBounds;

	OutData.StreamableMips.Lock(LOCK_READ_WRITE);
	uint8* Ptr = (uint8*)OutData.StreamableMips.Realloc(StreamableMipData.Num());
	FMemory::Memcpy(Ptr, StreamableMipData.GetData(), StreamableMipData.Num());
	OutData.StreamableMips.Unlock();
	OutData.StreamableMips.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
}

void FMeshUtilities::GenerateSignedDistanceFieldVolumeData(
	FString MeshName,
	const FMeshDataForDerivedDataTask& MeshData,
	float DistanceFieldResolutionScale,
	bool bGenerateAsIfTwoSided,
	class FDistanceFieldVolumeData& OutData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateSignedDistanceFieldVolumeData);
	COOK_STAT(auto Timer = DistanceFieldUtilityCookStats::UsageStats.TimeSyncWork());
	COOK_STAT(Timer.TrackCyclesOnly());

	if (DistanceFieldResolutionScale > 0)
	{
		const double StartTime = FPlatformTime::Seconds();
		const bool bIncludeTranslucentTriangles = false;

		FEmbreeScene EmbreeScene;
		MeshRepresentation::SetupEmbreeScene(MeshName, bGenerateAsIfTwoSided, EmbreeScene);

		// If Embree setup fails, there will be no scene to operate on. Early out.
		if (!EmbreeScene.Scene)
		{
			return;
		}

		if (!MeshRepresentation::AddMeshDataToEmbreeScene(EmbreeScene, MeshData, bIncludeTranslucentTriangles))
		{
			return;
		}

		EmbreeScene.Commit();

		BuildSignedDistanceField(MeshName, EmbreeScene, MeshData.Bounds, DistanceFieldResolutionScale, OutData);

		MeshRepresentation::DeleteEmbreeScene(EmbreeScene);

		const float BuildTime = (float)(FPlatformTime::Seconds() - StartTime);
		 
		if (BuildTime > 1.0f)
		{
			const FInt32Vector Mip0IndirectionDimensions = OutData.Mips[0].IndirectionDimensions;

			UE_LOGF(LogMeshUtilities, Log, "Finished distance field build in %.1fs - %ux%ux%u sparse distance field, %.1fMb total, %.1fMb always loaded, %u%% occupied, %u triangles, %ls",
				BuildTime,
				Mip0IndirectionDimensions.X * DistanceField::UniqueDataBrickSize,
				Mip0IndirectionDimensions.Y * DistanceField::UniqueDataBrickSize,
				Mip0IndirectionDimensions.Z * DistanceField::UniqueDataBrickSize,
				(OutData.GetResourceSizeBytes() + OutData.StreamableMips.GetBulkDataSize()) / 1024.0f / 1024.0f,
				(OutData.AlwaysLoadedMip.GetAllocatedSize()) / 1024.0f / 1024.0f,
				FMath::RoundToInt(100.0f * OutData.Mips[0].NumDistanceFieldBricks / (float)(Mip0IndirectionDimensions.X * Mip0IndirectionDimensions.Y * Mip0IndirectionDimensions.Z)),
				EmbreeScene.NumTrianglesTotal,
				*MeshName);
		}
	}
}

#else

void FMeshUtilities::GenerateSignedDistanceFieldVolumeData(
	FString MeshName,
	const FMeshDataForDerivedDataTask& MeshData,
	float DistanceFieldResolutionScale,
	bool bGenerateAsIfTwoSided,
	class FDistanceFieldVolumeData& OutData)
{
	if (DistanceFieldResolutionScale > 0)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Couldn't generate distance field for mesh, platform is missing Embree support.");
	}
}

#endif // PLATFORM_ENABLE_VECTORINTRINSICS
