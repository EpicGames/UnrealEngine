// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshRepresentationCommon.h"

#include "HAL/PlatformMemory.h"
#include "MaterialShared.h"
#include "MeshUtilities.h"
#include "MeshUtilitiesPrivate.h"
#include "DerivedMeshDataTaskUtils.h"

#if USE_EMBREE

static TAutoConsoleVariable<int32> CVarMemoryEstimateFactor(
	TEXT("r.Embree.MemoryEstimateFactor"), 60,
	TEXT("Configurable ratio for the memory used by embree"),
	ECVF_ReadOnly
); 

#endif


static FVector3f UniformSampleHemisphere(FVector2D Uniforms)
{
	Uniforms = Uniforms * 2.0f - 1.0f;

	if (Uniforms == FVector2D::ZeroVector)
	{
		return FVector3f::ZeroVector;
	}

	float R;
	float Theta;

	if (FMath::Abs(Uniforms.X) > FMath::Abs(Uniforms.Y))
	{
		R = Uniforms.X;
		Theta = (float)PI / 4 * (Uniforms.Y / Uniforms.X);
	}
	else
	{
		R = Uniforms.Y;
		Theta = (float)PI / 2 - (float)PI / 4 * (Uniforms.X / Uniforms.Y);
	}

	// concentric disk sample
	const float U = R * FMath::Cos(Theta);
	const float V = R * FMath::Sin(Theta);
	const float R2 = R * R;

	// map to hemisphere [P. Shirley, Kenneth Chiu; 1997; A Low Distortion Map Between Disk and Square]
	return FVector3f(U * FMath::Sqrt(2 - R2), V * FMath::Sqrt(2 - R2), 1.0f - R2);
}

void MeshUtilities::GenerateStratifiedUniformHemisphereSamples(int32 NumSamples, FRandomStream& RandomStream, TArray<FVector3f>& Samples)
{
	const int32 NumSamplesDim = FMath::TruncToInt(FMath::Sqrt((float)NumSamples));

	Samples.Empty(NumSamplesDim * NumSamplesDim);

	for (int32 IndexX = 0; IndexX < NumSamplesDim; IndexX++)
	{
		for (int32 IndexY = 0; IndexY < NumSamplesDim; IndexY++)
		{
			const float U1 = RandomStream.GetFraction();
			const float U2 = RandomStream.GetFraction();

			const float Fraction1 = (IndexX + U1) / (float)NumSamplesDim;
			const float Fraction2 = (IndexY + U2) / (float)NumSamplesDim;

			FVector3f Tmp = UniformSampleHemisphere(FVector2D(Fraction1, Fraction2));

			// Workaround issue with compiler optimization by using copy constructor here.
			Samples.Add(FVector3f(Tmp));
		}
	}
}

#if USE_EMBREE
void EmbreeFilterFunc(const struct RTCFilterFunctionNArguments* args)
{
	const FEmbreeGeometryAsset* GeometryAsset = (const FEmbreeGeometryAsset*)args->geometryUserPtr;
	FEmbreeTriangleDesc Desc = GeometryAsset->TriangleDescs[RTCHitN_primID(args->hit, 1, 0)];

#if USE_EMBREE_MAJOR_VERSION >= 4
	FEmbreeRayQueryContext& EmbreeContext = *static_cast<FEmbreeRayQueryContext*>(args->context);
#else
	FEmbreeIntersectionContext& EmbreeContext = *static_cast<FEmbreeIntersectionContext*>(args->context);
#endif
	EmbreeContext.ElementIndex = Desc.ElementIndex;

	const RTCHit& EmbreeHit = *(RTCHit*)args->hit;
	if (EmbreeContext.SkipPrimId != RTC_INVALID_GEOMETRY_ID && EmbreeContext.SkipPrimId == EmbreeHit.primID)
	{
		// Ignore hit in order to continue tracing
		args->valid[0] = 0;
	}
}

void EmbreeErrorFunc(void* userPtr, RTCError code, const char* str)
{
	FString ErrorString;
	TArray<TCHAR, FString::AllocatorType>& ErrorStringArray = ErrorString.GetCharArray();
	ErrorStringArray.Empty();

	int32 StrLen = FCStringAnsi::Strlen(str);
	int32 Length = FUTF8ToTCHAR_Convert::ConvertedLength(str, StrLen);
	ErrorStringArray.AddUninitialized(Length + 1); // +1 for the null terminator
	FUTF8ToTCHAR_Convert::Convert(ErrorStringArray.GetData(), ErrorStringArray.Num(), reinterpret_cast<const ANSICHAR*>(str), StrLen);
	ErrorStringArray[Length] = TEXT('\0');

	UE_LOGF(LogMeshUtilities, Error, "Embree error: %ls Code=%u", *ErrorString, (uint32)code);
}

LLM_DEFINE_TAG(Embree);

// userPtr is provided during callback registration, it points to the associated FEmbreeScene
static bool EmbreeMemoryMonitor(void* userPtr, ssize_t bytes, bool post)
{
	LLM_SCOPE_BYTAG(Embree);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Default, static_cast<int64>(bytes)));

	return true;
}
#endif

int64_t MeshRepresentation::MemoryEstimateForEmbreeScene(uint64_t IndexCount)
{
#if USE_EMBREE
	// This value was observed by breakpointing VmAlloc and viewing the allocations made
	const int64_t EmbreeDeviceUsage = 1024 * 1024 * 16; 

	// Estimate based of measuring and correlating actual memory usage with various mesh properties. 
	// Strong correlation between number of indices and memory usage. 1mb bias added to catch edge cases with very low index counts.
	return (CVarMemoryEstimateFactor.GetValueOnAnyThread() * IndexCount) + (1024 * 1024) + EmbreeDeviceUsage;
#else
	return 0;
#endif
}

void MeshRepresentation::SetupEmbreeScene(FString MeshName, bool bGenerateAsIfTwoSided, FEmbreeScene& OutEmbreeScene)
{
	OutEmbreeScene.MeshName = MeshName;
	OutEmbreeScene.bGenerateAsIfTwoSided = bGenerateAsIfTwoSided;

#if USE_EMBREE
	OutEmbreeScene.Device = rtcNewDevice(nullptr);
	rtcSetDeviceErrorFunction(OutEmbreeScene.Device, EmbreeErrorFunc, nullptr);
	LLM_IF_ENABLED(rtcSetDeviceMemoryMonitorFunction(OutEmbreeScene.Device, EmbreeMemoryMonitor, &OutEmbreeScene));

	RTCError ReturnErrorNewDevice = rtcGetDeviceError(OutEmbreeScene.Device);
	if (ReturnErrorNewDevice == RTC_ERROR_OUT_OF_MEMORY)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to created Embree device for %ls (OUT_OF_MEMORY).", *MeshName);
		FPlatformMemory::OnOutOfMemory(0, 16);
	}
	if (ReturnErrorNewDevice != RTC_ERROR_NONE)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to created Embree device for %ls. Code: %d", *MeshName, (int32)ReturnErrorNewDevice);
		return;
	}

	OutEmbreeScene.Scene = rtcNewScene(OutEmbreeScene.Device);
	rtcSetSceneFlags(OutEmbreeScene.Scene, RTC_SCENE_FLAG_NONE);

	RTCError ReturnErrorNewScene = rtcGetDeviceError(OutEmbreeScene.Device);
	if (ReturnErrorNewScene == RTC_ERROR_OUT_OF_MEMORY)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to created Embree scene for %ls (OUT_OF_MEMORY).", *MeshName);
		FPlatformMemory::OnOutOfMemory(0, 16);
	}
	if (ReturnErrorNewScene != RTC_ERROR_NONE)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to created Embree scene for %ls. Code: %d", *MeshName, (int32)ReturnErrorNewScene);
		rtcReleaseDevice(OutEmbreeScene.Device);
		return;
	}
#endif
}

void MeshRepresentation::DeleteEmbreeScene(FEmbreeScene& EmbreeScene)
{
#if USE_EMBREE
	for (FEmbreeGeometryAsset& Asset : EmbreeScene.GeometryAssets)
	{
		if (Asset.Scene != nullptr)
		{
			rtcReleaseScene(Asset.Scene);
		}
	}

	rtcReleaseScene(EmbreeScene.Scene);
	rtcReleaseDevice(EmbreeScene.Device);
#endif

	EmbreeScene = FEmbreeScene();
}

bool MeshRepresentation::AddMeshDataToEmbreeScene(FEmbreeScene& EmbreeScene, const FMeshDataForDerivedDataTask& MeshData, bool bIncludeTranslucentTriangles)
{
	if ((MeshData.SourceMeshData == nullptr || !MeshData.SourceMeshData->IsValid()) && MeshData.LODModel == nullptr)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Provided MeshData for %ls doesn't contain any data.", *EmbreeScene.MeshName);
		return false;
	}

	const FEmbreeGeometryAsset* GeometryAsset = EmbreeScene.AddGeometryAsset(MeshData.SourceMeshData, MeshData.LODModel, MeshData.SectionData, bIncludeTranslucentTriangles, false);

	const FEmbreeGeometry* Geometry = EmbreeScene.AddGeometry(GeometryAsset);

	TArray<const FEmbreeGeometryAsset*> PartGeometryAssets;
	PartGeometryAssets.Reserve(MeshData.Parts.Num());

	for (const FSignedDistanceFieldBuildPartData& Part : MeshData.Parts)
	{
		if (Part.SourceMeshData.IsValid())
		{
			PartGeometryAssets.Add(EmbreeScene.AddGeometryAsset(&Part.SourceMeshData, nullptr, Part.SectionData, bIncludeTranslucentTriangles, true));
		}
		else
		{
			PartGeometryAssets.Add(nullptr);
		}
	}

	for (const FNaniteAssemblyNode& Node : MeshData.Nodes)
	{
		if (PartGeometryAssets[Node.PartIndex] != nullptr)
		{
			const FEmbreeGeometry* PartGeometryInstance = EmbreeScene.AddGeometryInstance(PartGeometryAssets[Node.PartIndex], Node.Transform.ToMatrixWithScale());
		}
		else
		{
			UE_LOGF(LogStaticMesh, Warning, "Nanite Assembly node ignored because corresponding part doesn't have a valid representation. See previous line(s) for details.");
		}
	}

	return true;
}

const FEmbreeGeometryAsset* FEmbreeScene::AddGeometryAsset(
	const FSourceMeshDataForDerivedDataTask* SourceMeshData,
	const FStaticMeshLODResources* LODModel,
	TConstArrayView<FSignedDistanceFieldBuildSectionData> SectionData,
	bool bIncludeTranslucentTriangles,
	bool bInstantiable)
{
	check((SourceMeshData != nullptr && SourceMeshData->IsValid()) || (LODModel != nullptr));

#if USE_EMBREE
	const uint32 NumVertices = (SourceMeshData && SourceMeshData->IsValid()) ? SourceMeshData->GetNumVertices() : LODModel->VertexBuffers.PositionVertexBuffer.GetNumVertices();
	const uint32 NumIndices = (SourceMeshData && SourceMeshData->IsValid()) ? SourceMeshData->GetNumIndices() : LODModel->IndexBuffer.GetNumIndices();
	const int32 NumTriangles = NumIndices / 3;

	const FStaticMeshSectionArray& Sections = (SourceMeshData && SourceMeshData->IsValid()) ? SourceMeshData->Sections : LODModel->Sections;

	TArray<int32> FilteredTriangles;
	FilteredTriangles.Empty(NumTriangles);

	// TODO: investigate/fix what causes this warning and then enable it
	bool bWarnOnceSectionData = false;

	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
	{
		FVector3f V0, V1, V2;

		if (SourceMeshData && SourceMeshData->IsValid())
		{
			const uint32 I0 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 0];
			const uint32 I1 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 1];
			const uint32 I2 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 2];

			V0 = SourceMeshData->VertexPositions[I0];
			V1 = SourceMeshData->VertexPositions[I1];
			V2 = SourceMeshData->VertexPositions[I2];
		}
		else
		{
			const FIndexArrayView Indices = LODModel->IndexBuffer.GetArrayView();
			const uint32 I0 = Indices[TriangleIndex * 3 + 0];
			const uint32 I1 = Indices[TriangleIndex * 3 + 1];
			const uint32 I2 = Indices[TriangleIndex * 3 + 2];

			V0 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I0);
			V1 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I1);
			V2 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I2);
		}

		const FVector3f TriangleNormal = ((V1 - V2) ^ (V0 - V2));
		const bool bDegenerateTriangle = TriangleNormal.SizeSquared() < SMALL_NUMBER;
		if (!bDegenerateTriangle)
		{
			bool bIncludeTriangle = false;

			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
			{
				const FStaticMeshSection& Section = Sections[SectionIndex];

				if ((uint32)(TriangleIndex * 3) >= Section.FirstIndex && (uint32)(TriangleIndex * 3) < Section.FirstIndex + Section.NumTriangles * 3)
				{
					if (SectionData.IsValidIndex(SectionIndex))
					{
						const bool bIsOpaqueOrMasked = !IsTranslucentBlendMode(SectionData[SectionIndex].BlendMode);
						bIncludeTriangle = (bIsOpaqueOrMasked || bIncludeTranslucentTriangles) && SectionData[SectionIndex].bAffectDistanceFieldLighting;
					}
					else if (bWarnOnceSectionData)
					{
						UE_LOGF(LogMeshUtilities, Warning, "Missing section data for %ls, section = %d.", *MeshName, SectionIndex);
						bWarnOnceSectionData = false;
					}

					break;
				}
			}

			if (bIncludeTriangle)
			{
				FilteredTriangles.Add(TriangleIndex);
			}
		}
	}

	FEmbreeGeometryAsset& GeometryAsset = *new FEmbreeGeometryAsset;

	const int32 NumBufferVerts = 1; // Reserve extra space at the end of the array, as embree has an internal bug where they read and discard 4 bytes off the end of the array

	GeometryAsset.VertexArray.Empty(NumVertices + NumBufferVerts);
	GeometryAsset.VertexArray.AddUninitialized(NumVertices + NumBufferVerts);

	const int32 NumFilteredIndices = FilteredTriangles.Num() * 3;

	GeometryAsset.IndexArray.Empty(NumFilteredIndices);
	GeometryAsset.IndexArray.AddUninitialized(NumFilteredIndices);

	GeometryAsset.TriangleDescs.Empty(FilteredTriangles.Num());

	FVector3f* EmbreeVertices = GeometryAsset.VertexArray.GetData();
	uint32* EmbreeIndices = GeometryAsset.IndexArray.GetData();

	for (int32 FilteredTriangleIndex = 0; FilteredTriangleIndex < FilteredTriangles.Num(); FilteredTriangleIndex++)
	{
		uint32 I0, I1, I2;
		FVector3f V0, V1, V2;

		const int32 TriangleIndex = FilteredTriangles[FilteredTriangleIndex];
		if (SourceMeshData && SourceMeshData->IsValid())
		{
			I0 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 0];
			I1 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 1];
			I2 = SourceMeshData->TriangleIndices[TriangleIndex * 3 + 2];

			V0 = SourceMeshData->VertexPositions[I0];
			V1 = SourceMeshData->VertexPositions[I1];
			V2 = SourceMeshData->VertexPositions[I2];
		}
		else
		{
			const FIndexArrayView Indices = LODModel->IndexBuffer.GetArrayView();
			I0 = Indices[TriangleIndex * 3 + 0];
			I1 = Indices[TriangleIndex * 3 + 1];
			I2 = Indices[TriangleIndex * 3 + 2];

			V0 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I0);
			V1 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I1);
			V2 = LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(I2);
		}

		bool bTriangleIsTwoSided = false;

		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
		{
			const FStaticMeshSection& Section = Sections[SectionIndex];

			if ((uint32)(TriangleIndex * 3) >= Section.FirstIndex && (uint32)(TriangleIndex * 3) < Section.FirstIndex + Section.NumTriangles * 3)
			{
				if (SectionData.IsValidIndex(SectionIndex))
				{
					bTriangleIsTwoSided = SectionData[SectionIndex].bTwoSided;
				}
				else if (bWarnOnceSectionData)
				{
					UE_LOGF(LogMeshUtilities, Warning, "Missing section data for %ls, section = %d.", *MeshName, SectionIndex);
					bWarnOnceSectionData = false;
				}

				break;
			}
		}

		{
			EmbreeIndices[FilteredTriangleIndex * 3 + 0] = I0;
			EmbreeIndices[FilteredTriangleIndex * 3 + 1] = I1;
			EmbreeIndices[FilteredTriangleIndex * 3 + 2] = I2;

			EmbreeVertices[I0] = V0;
			EmbreeVertices[I1] = V1;
			EmbreeVertices[I2] = V2;

			FEmbreeTriangleDesc Desc;
			// Store bGenerateAsIfTwoSided in material index
			Desc.ElementIndex = bGenerateAsIfTwoSided || bTriangleIsTwoSided ? 1 : 0;
			GeometryAsset.TriangleDescs.Add(Desc);
		}
	}

	{
		GeometryAsset.NumVertices = NumVertices;
		GeometryAsset.NumTriangles = FilteredTriangles.Num();

		GeometryAsset.SectionNumTriangles = 0;
		GeometryAsset.SectionNumTwoSidedTriangles = 0;

		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
		{
			const FStaticMeshSection& Section = Sections[SectionIndex];

			if (SectionData.IsValidIndex(SectionIndex))
			{
				GeometryAsset.SectionNumTriangles += Section.NumTriangles;

				if (SectionData[SectionIndex].bTwoSided)
				{
					GeometryAsset.SectionNumTwoSidedTriangles += Section.NumTriangles;
				}
			}
			else if (bWarnOnceSectionData)
			{
				UE_LOGF(LogMeshUtilities, Warning, "Missing section data for %ls, section = %d.", *MeshName, SectionIndex);
				bWarnOnceSectionData = false;
			}
		}
	}

	if (bInstantiable)
	{
		RTCGeometry ImplGeometry = rtcNewGeometry(Device, RTC_GEOMETRY_TYPE_TRIANGLE);

		rtcSetSharedGeometryBuffer(ImplGeometry, RTC_BUFFER_TYPE_VERTEX, 0, RTC_FORMAT_FLOAT3, EmbreeVertices, 0, sizeof(FVector3f), GeometryAsset.NumVertices);
		rtcSetSharedGeometryBuffer(ImplGeometry, RTC_BUFFER_TYPE_INDEX, 0, RTC_FORMAT_UINT3, EmbreeIndices, 0, sizeof(uint32) * 3, GeometryAsset.NumTriangles);

		rtcSetGeometryUserData(ImplGeometry, &GeometryAsset);
		rtcSetGeometryIntersectFilterFunction(ImplGeometry, EmbreeFilterFunc);

		rtcCommitGeometry(ImplGeometry);

		GeometryAsset.Scene = rtcNewScene(Device);

		rtcAttachGeometry(GeometryAsset.Scene, ImplGeometry);
		rtcReleaseGeometry(ImplGeometry);

		rtcCommitScene(GeometryAsset.Scene);

		RTCError ReturnError = rtcGetDeviceError(Device);
		if (ReturnError == RTC_ERROR_OUT_OF_MEMORY)
		{
			UE_LOGF(LogMeshUtilities, Warning, "Failed to create instantiable Embree geometry for %ls (OUT_OF_MEMORY).", *MeshName);
			FPlatformMemory::OnOutOfMemory(0, 16);
			//return nullptr; // unreachable code
		}
		if (ReturnError != RTC_ERROR_NONE)
		{
			UE_LOGF(LogMeshUtilities, Warning, "Failed to create instantiable Embree geometry for %ls. Code: %d", *MeshName, (int32)ReturnError);
			return nullptr;
		}
	}

	GeometryAssets.Add(&GeometryAsset);

	return &GeometryAsset;
#else
	return nullptr;
#endif
}

const FEmbreeGeometry* FEmbreeScene::AddGeometry(const FEmbreeGeometryAsset* GeometryAsset)
{
#if USE_EMBREE
	const FVector3f* VerticesData = GeometryAsset->VertexArray.GetData();
	const uint32* IndicesData = GeometryAsset->IndexArray.GetData();

	RTCGeometry ImplGeometry = rtcNewGeometry(Device, RTC_GEOMETRY_TYPE_TRIANGLE);

	rtcSetSharedGeometryBuffer(ImplGeometry, RTC_BUFFER_TYPE_VERTEX, 0, RTC_FORMAT_FLOAT3, VerticesData, 0, sizeof(FVector3f), GeometryAsset->NumVertices);
	rtcSetSharedGeometryBuffer(ImplGeometry, RTC_BUFFER_TYPE_INDEX, 0, RTC_FORMAT_UINT3, IndicesData, 0, sizeof(uint32) * 3, GeometryAsset->NumTriangles);

	rtcSetGeometryUserData(ImplGeometry, (void*)GeometryAsset);
	rtcSetGeometryIntersectFilterFunction(ImplGeometry, EmbreeFilterFunc);

	rtcCommitGeometry(ImplGeometry);

	uint32 GeometryId = rtcAttachGeometry(Scene, ImplGeometry);
	rtcReleaseGeometry(ImplGeometry);

	RTCError ReturnError = rtcGetDeviceError(Device);
	if (ReturnError == RTC_ERROR_OUT_OF_MEMORY)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to add geometry to Embree scene for %ls (OUT_OF_MEMORY).", *MeshName);
		FPlatformMemory::OnOutOfMemory(0, 16);
		//return nullptr; // unreachable code
	}
	if (ReturnError != RTC_ERROR_NONE)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to add geometry to Embree scene for %ls. Code: %d", *MeshName, (int32)ReturnError);
		return nullptr;
	}

	FEmbreeGeometry& Geometry = *new FEmbreeGeometry;
	Geometry.Asset = GeometryAsset;
	Geometry.GeometryId = GeometryId;

	Geometries.Add(&Geometry);

	return &Geometry;
#else
	return nullptr;
#endif
}

const FEmbreeGeometry* FEmbreeScene::AddGeometryInstance(const FEmbreeGeometryAsset* GeometryAsset, const FMatrix44f& Transform)
{
#if USE_EMBREE

	RTCGeometry ImplGeometry = rtcNewGeometry(Device, RTC_GEOMETRY_TYPE_INSTANCE);

	rtcSetGeometryInstancedScene(ImplGeometry, GeometryAsset->Scene);
	rtcSetGeometryTransform(ImplGeometry, 0, RTC_FORMAT_FLOAT4X4_COLUMN_MAJOR, (const float*)Transform.M);
	rtcCommitGeometry(ImplGeometry);

	uint32 GeometryId = rtcAttachGeometry(Scene, ImplGeometry);
	rtcReleaseGeometry(ImplGeometry);

	RTCError ReturnError = rtcGetDeviceError(Device);
	if (ReturnError == RTC_ERROR_OUT_OF_MEMORY)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to add geometry instance to Embree scene for %ls (OUT_OF_MEMORY).", *MeshName);
		FPlatformMemory::OnOutOfMemory(0, 16);
		//return nullptr; // unreachable code
	}
	if (ReturnError != RTC_ERROR_NONE)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to add geometry instance to Embree scene for %ls. Code: %d", *MeshName, (int32)ReturnError);
		return nullptr;
	}

	FEmbreeGeometry& Geometry = *new FEmbreeGeometry;
	Geometry.Asset = GeometryAsset;
	Geometry.GeometryId = GeometryId;

	Geometries.Add(&Geometry);

	return &Geometry;
#else
	return nullptr;
#endif
}

void FEmbreeScene::Commit()
{
#if USE_EMBREE

	NumTrianglesTotal = 0;

	uint32 SectionNumTwoSidedTriangles = 0;
	uint32 SectionNumTriangles = 0;

	for (FEmbreeGeometry& Geometry : Geometries)
	{
		NumTrianglesTotal += Geometry.Asset->NumTriangles;

		SectionNumTwoSidedTriangles += Geometry.Asset->SectionNumTwoSidedTriangles;
		SectionNumTriangles += Geometry.Asset->SectionNumTriangles;
	}

	bMostlyTwoSided = SectionNumTwoSidedTriangles * 4 >= SectionNumTriangles || bGenerateAsIfTwoSided;

	rtcCommitScene(Scene);

	RTCError ReturnError = rtcGetDeviceError(Device);
	if (ReturnError == RTC_ERROR_OUT_OF_MEMORY)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to commit Embree scene for %ls (OUT_OF_MEMORY).", *MeshName);
		FPlatformMemory::OnOutOfMemory(0, 16);
	}
	if (ReturnError != RTC_ERROR_NONE)
	{
		UE_LOGF(LogMeshUtilities, Warning, "Failed to commit Embree scene for %ls. Code: %d", *MeshName, (int32)ReturnError);
		return;
	}
#endif
}