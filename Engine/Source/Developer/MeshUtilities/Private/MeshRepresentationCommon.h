// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshUtilities.h"

#if USE_EMBREE
#if USE_EMBREE_MAJOR_VERSION >= 4
	#include <embree4/rtcore.h>
	#include <embree4/rtcore_ray.h>
#else
	#include <embree3/rtcore.h>
	#include <embree3/rtcore_ray.h>
#endif
#else
	#define RTC_INVALID_GEOMETRY_ID ((unsigned int)-1)
	typedef void* RTCDevice;
	typedef void* RTCScene;
	typedef void* RTCGeometry;
#endif

class FSourceMeshDataForDerivedDataTask;

struct FEmbreeTriangleDesc
{
	int16 ElementIndex;

	bool IsTwoSided() const
	{
		// MaterialIndex on the build triangles was set to 1 if two-sided, or 0 if one-sided
		return ElementIndex == 1;
	}
};

struct FEmbreeGeometryAsset
{
	TArray<uint32> IndexArray;
	TArray<FVector3f> VertexArray;
	TArray<FEmbreeTriangleDesc> TriangleDescs; // The material ID of each triangle.

	uint32 NumVertices;
	uint32 NumTriangles;

	uint32 SectionNumTwoSidedTriangles;
	uint32 SectionNumTriangles;

	RTCScene Scene = nullptr;
};

struct FEmbreeGeometry
{
	const FEmbreeGeometryAsset* Asset;
	uint32 GeometryId = RTC_INVALID_GEOMETRY_ID;
};

class FEmbreeScene
{
public:

	const FEmbreeGeometryAsset* AddGeometryAsset(
		const FSourceMeshDataForDerivedDataTask* SourceMeshData,
		const FStaticMeshLODResources* LODModel,
		TConstArrayView<FSignedDistanceFieldBuildSectionData> SectionData,
		bool bIncludeTranslucentTriangles,
		bool bInstantiable);

	const FEmbreeGeometry* AddGeometry(const FEmbreeGeometryAsset* GeometryAsset);

	const FEmbreeGeometry* AddGeometryInstance(const FEmbreeGeometryAsset* GeometryAsset, const FMatrix44f& Transform);

	void Commit();

	FString MeshName;
	bool bGenerateAsIfTwoSided = false;

	RTCDevice Device = nullptr;
	RTCScene Scene = nullptr;

	TIndirectArray<FEmbreeGeometryAsset> GeometryAssets;
	TIndirectArray<FEmbreeGeometry> Geometries;

	int32 NumTrianglesTotal = 0;
	bool bMostlyTwoSided = false;
};

#if USE_EMBREE
struct FEmbreeRay : public RTCRayHit
{
	FEmbreeRay() :
		ElementIndex(-1)
	{
		hit.u = hit.v = 0;
		ray.time = 0;
		ray.mask = 0xFFFFFFFF;
		hit.geomID = RTC_INVALID_GEOMETRY_ID;
		hit.instID[0] = RTC_INVALID_GEOMETRY_ID;
		hit.primID = RTC_INVALID_GEOMETRY_ID;
	}

	FVector3f GetHitNormal() const
	{
		return FVector3f(-hit.Ng_x, -hit.Ng_y, -hit.Ng_z).GetSafeNormal();
	}	

	bool IsHitTwoSided() const
	{
		// MaterialIndex on the build triangles was set to 1 if two-sided, or 0 if one-sided
		return ElementIndex == 1;
	}

	// Additional Outputs.
	int32 ElementIndex; // Material Index
};

#if USE_EMBREE_MAJOR_VERSION >= 4
struct FEmbreeRayQueryContext : public RTCRayQueryContext
{
	FEmbreeRayQueryContext() :
		ElementIndex(-1)
	{}
#else
struct FEmbreeIntersectionContext : public RTCIntersectContext
{
	FEmbreeIntersectionContext() :
		ElementIndex(-1)
	{}
#endif

	bool IsHitTwoSided() const
	{
		// MaterialIndex on the build triangles was set to 1 if two-sided, or 0 if one-sided
		return ElementIndex == 1;
	}

	// Hit against this primitive will be ignored
	int32 SkipPrimId = RTC_INVALID_GEOMETRY_ID;

	// Additional Outputs.
	int32 ElementIndex; // Material Index
};

#endif

namespace MeshRepresentation
{

	void SetupEmbreeScene(FString MeshName, bool bGenerateAsIfTwoSided, FEmbreeScene& OutEmbreeScene);

	void DeleteEmbreeScene(FEmbreeScene& EmbreeScene);

	int64_t MemoryEstimateForEmbreeScene(uint64_t IndexCount);

	bool AddMeshDataToEmbreeScene(FEmbreeScene& EmbreeScene, const FMeshDataForDerivedDataTask& MeshData, bool bIncludeTranslucentTriangles);

};
