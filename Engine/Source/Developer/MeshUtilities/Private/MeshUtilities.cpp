// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshUtilities.h"
#include "MeshUtilitiesPrivate.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopeLock.h"
#include "Containers/Ticker.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Textures/SlateIcon.h"
#include "Styling/SlateTypes.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "SkeletalMeshToolMenuContext.h"
#include "Components/MeshComponent.h"
#include "RawIndexBuffer.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/ShapeComponent.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "EditorDirectories.h"
#include "Materials/Material.h"
#include "RawMesh.h"
#include "StaticMeshResources.h"
#include "MeshBuild.h"
#include "ThirdPartyBuildOptimizationHelper.h"
#include "SkeletalMeshTools.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkinnedMeshComponent.h"
#include "ImageUtils.h"
#include "LayoutUV.h"
#include "mikktspace.h"
#include "Misc/FbxErrors.h"
#include "Components/SplineMeshComponent.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "MaterialUtilities.h"
#include "MaterialShared.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "MeshBoneReduction.h"
#include "MeshMergeData.h"
#include "GPUSkinVertexFactory.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/Character.h"
#include "Components/CapsuleComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Algo/Transform.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Math/Bounds.h"

#include "LandscapeProxy.h"
#include "Landscape.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "Engine/MeshMergeCullingVolume.h"

#include "LevelEditor.h"
#include "IAnimationBlueprintEditor.h"
#include "IAnimationBlueprintEditorModule.h"
#include "IAnimationEditor.h"
#include "IAnimationEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ISkeletalMeshEditor.h"
#include "ISkeletalMeshEditorModule.h"
#include "ISkeletonEditor.h"
#include "ISkeletonEditorModule.h"
#include "IPersonaToolkit.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "SkeletalRenderPublic.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/MeshSimplificationSettings.h"
#include "Engine/SkeletalMeshSimplificationSettings.h"
#include "Engine/ProxyLODMeshSimplificationSettings.h"

#include "Editor/EditorPerProjectUserSettings.h"
#include "IDetailCustomization.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "OverlappingCorners.h"
#include "MeshUtilitiesCommon.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "MeshUtilitiesEngine.h"

#include "LODUtilities.h"
#include "ClothingAsset.h"

#if WITH_EDITOR
#include "Editor.h"
#include "UnrealEdMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif
#include "BoneWeights.h"
#include "MaterialBakingStructures.h"
#include "IMaterialBakingModule.h"
#include "MaterialOptions.h"


#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneInfo.h"
#include "IMeshReductionManagerModule.h"
#include "MeshMergeModule.h"

#include "Rendering/SkinWeightProfile.h"
#include "PersonaToolMenuContext.h"

DEFINE_LOG_CATEGORY(LogMeshUtilities);
/*------------------------------------------------------------------------------
MeshUtilities module.
------------------------------------------------------------------------------*/

// The version string is a GUID. If you make a change to mesh utilities that
// causes meshes to be rebuilt you MUST generate a new GUID and replace this
// string with it.

#define MESH_UTILITIES_VER TEXT("228332BAE0224DD294E232B87D83948F")

#define LOCTEXT_NAMESPACE "MeshUtils"

IMPLEMENT_MODULE(FMeshUtilities, MeshUtilities);

void FMeshUtilities::CacheOptimizeIndexBuffer(TArray<uint16>& Indices)
{
	BuildOptimizationThirdParty::CacheOptimizeIndexBuffer(Indices);
}

void FMeshUtilities::CacheOptimizeIndexBuffer(TArray<uint32>& Indices)
{
	BuildOptimizationThirdParty::CacheOptimizeIndexBuffer(Indices);
}

void FMeshUtilities::BuildSkeletalAdjacencyIndexBuffer(
	const TArray<FSoftSkinVertex>& VertexBuffer,
	const uint32 TexCoordCount,
	const TArray<uint32>& Indices,
	TArray<uint32>& OutPnAenIndices
	)
{
	BuildOptimizationThirdParty::NvTriStripHelper::BuildSkeletalAdjacencyIndexBuffer(VertexBuffer, TexCoordCount, Indices, OutPnAenIndices);
}

void CalculateTriangleTangentInternal(const FVector3f& VertexPosA, const FVector2f& VertexUVA
									  , const FVector3f& VertexPosB, const FVector2f& VertexUVB
									  , const FVector3f& VertexPosC, const FVector2f& VertexUVC
									  , TArray<FVector3f>& OutTangents, float CompareThreshold)
{
	//We must always allocate the OutTangents to 3 FVector3f
	OutTangents.Reset(3);
	OutTangents.AddZeroed(3);
	const FVector3f Positions[3] = { VertexPosA, VertexPosB, VertexPosC };
	const FVector3f Normal = ((Positions[1] - Positions[2]) ^ (Positions[0] - Positions[2])).GetSafeNormal(CompareThreshold);
	//Avoid doing orthonormal vector from a degenerated triangle.
	if (!Normal.IsNearlyZero(FLT_MIN))
	{
		FMatrix44f	ParameterToLocal(
			FPlane4f(Positions[1].X - Positions[0].X, Positions[1].Y - Positions[0].Y, Positions[1].Z - Positions[0].Z, 0),
			FPlane4f(Positions[2].X - Positions[0].X, Positions[2].Y - Positions[0].Y, Positions[2].Z - Positions[0].Z, 0),
			FPlane4f(Positions[0].X, Positions[0].Y, Positions[0].Z, 0),
			FPlane4f(0, 0, 0, 1)
		);

		const FVector2f T1 = VertexUVA;
		const FVector2f T2 = VertexUVB;
		const FVector2f T3 = VertexUVC;

		FMatrix44f ParameterToTexture(
			FPlane4f(T2.X - T1.X, T2.Y - T1.Y, 0, 0),
			FPlane4f(T3.X - T1.X, T3.Y - T1.Y, 0, 0),
			FPlane4f(T1.X, T1.Y, 1, 0),
			FPlane4f(0, 0, 0, 1)
		);

		if (!FMath::IsNearlyZero(ParameterToTexture.Determinant()))
		{
			// Use InverseSlow to catch singular matrices.  Inverse can miss this sometimes.
			const FMatrix44f TextureToLocal = ParameterToTexture.Inverse() * ParameterToLocal;

			OutTangents[0] = (TextureToLocal.TransformVector(FVector3f(1, 0, 0)).GetSafeNormal());
			OutTangents[1] = (TextureToLocal.TransformVector(FVector3f(0, 1, 0)).GetSafeNormal());
		}
		else
		{
			// Use the Duff & Frisvad algorithm (see Duff 2017 in JCGT) to construct a consistent tangent from a
			// single normal vector. 
			if (Normal.Z < 0.0f)
			{
				const float A = 1.0f / (1.0f - Normal.Z);
				const float B = Normal.X * Normal.Y * A;

				OutTangents[0] = FVector3f(1.0f - Normal.X * Normal.X * A, -B, Normal.X);
				OutTangents[1] = FVector3f(B, Normal.Y * Normal.Y * A - 1.0f, -Normal.Y);
			}
			else
			{
				const float A = 1.0f / (1.0f + Normal.Z);
				const float B = -Normal.X * Normal.Y * A;

				OutTangents[0] = FVector3f(1.0f - Normal.X * Normal.X * A, B, -Normal.X);
				OutTangents[1] = FVector3f(B, 1.0f - Normal.Y * Normal.Y * A, -Normal.Y);
			}
		}

		OutTangents[2] = (Normal);

		FVector3f::CreateOrthonormalBasis(
			OutTangents[0],
			OutTangents[1],
			OutTangents[2]
		);

		if (OutTangents[0].IsNearlyZero() || OutTangents[0].ContainsNaN()
			|| OutTangents[1].IsNearlyZero() || OutTangents[1].ContainsNaN())
		{
			OutTangents[0] = FVector3f::ZeroVector;
			OutTangents[1] = FVector3f::ZeroVector;
		}

		if (OutTangents[2].IsNearlyZero() || OutTangents[2].ContainsNaN())
		{
			OutTangents[2] = FVector3f::ZeroVector;
		}
	}
	else
	{
		//Add zero tangents and normal for this triangle, this is like weighting it to zero when we compute the vertex normal
		//But we need the triangle to correctly connect other neighbourg triangles
		OutTangents[0] = (FVector3f::ZeroVector);
		OutTangents[1] = (FVector3f::ZeroVector);
		OutTangents[2] = (FVector3f::ZeroVector);
	}
}

void FMeshUtilities::CalculateTriangleTangent(const FSoftSkinVertex& VertexA, const FSoftSkinVertex& VertexB, const FSoftSkinVertex& VertexC, TArray<FVector3f>& OutTangents, float CompareThreshold)
{
	CalculateTriangleTangentInternal(VertexA.Position, VertexA.UVs[0]
									 , VertexB.Position, VertexB.UVs[0]
									 , VertexC.Position, VertexC.UVs[0]
									 , OutTangents, CompareThreshold);
}

void FMeshUtilities::CalcBoneVertInfos(USkeletalMesh* SkeletalMesh, TArray<FBoneVertInfo>& Infos, bool bOnlyDominant, int32 LodIndex/* = 0*/)
{
	FMeshUtilitiesEngine::CalcBoneVertInfos(SkeletalMesh, Infos, bOnlyDominant, LodIndex);
}

// Helper function for ConvertMeshesToStaticMesh
static void AddOrDuplicateMaterial(UMaterialInterface* InMaterialInterface, const FString& InPackageName, TArray<UMaterialInterface*>& OutMaterials)
{
	if (InMaterialInterface && !InMaterialInterface->GetOuter()->IsA<UPackage>())
	{
		// Convert runtime material instances to new concrete material instances
		// Create new package
		FString OriginalMaterialName = InMaterialInterface->GetName();
		FString MaterialPath = FPackageName::GetLongPackagePath(InPackageName) / OriginalMaterialName;
		FString MaterialName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(MaterialPath, TEXT(""), MaterialPath, MaterialName);
		UPackage* MaterialPackage = CreatePackage( *MaterialPath);

		// Duplicate the object into the new package
		UMaterialInterface* NewMaterialInterface = DuplicateObject<UMaterialInterface>(InMaterialInterface, MaterialPackage, *MaterialName);
		NewMaterialInterface->SetFlags(RF_Public | RF_Standalone);

		if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(NewMaterialInterface))
		{
			UMaterialInstanceDynamic* OldMaterialInstanceDynamic = CastChecked<UMaterialInstanceDynamic>(InMaterialInterface);
			MaterialInstanceDynamic->K2_CopyMaterialInstanceParameters(OldMaterialInstanceDynamic);
		}

		NewMaterialInterface->MarkPackageDirty();

		FAssetRegistryModule::AssetCreated(NewMaterialInterface);

		InMaterialInterface = NewMaterialInterface;
	}

	OutMaterials.Add(InMaterialInterface);
}

// Helper function for ConvertMeshesToStaticMesh
template <typename ComponentType>
static void ProcessMaterials(ComponentType* InComponent, const FString& InPackageName, TArray<UMaterialInterface*>& OutMaterials)
{
	const int32 NumMaterials = InComponent->GetNumMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
	{
		UMaterialInterface* MaterialInterface = InComponent->GetMaterial(MaterialIndex);
		AddOrDuplicateMaterial(MaterialInterface, InPackageName, OutMaterials);
	}
}

// Helper function for ConvertMeshesToStaticMesh
static bool IsValidSkinnedMeshComponent(USkinnedMeshComponent* InComponent)
{
	return InComponent && InComponent->MeshObject && InComponent->IsVisible();
}

/** Helper struct for tracking validity of optional buffers */
struct FRawMeshTracker
{
	FRawMeshTracker()
		: bValidColors(false)
	{
		FMemory::Memset(bValidTexCoords, 0);
	}

	bool bValidTexCoords[MAX_MESH_TEXTURE_COORDS];
	bool bValidColors;
};

// Helper function for ConvertMeshesToStaticMesh
static void SkinnedMeshToRawMeshes(USkinnedMeshComponent* InSkinnedMeshComponent, int32 InOverallMaxLODs, const FMatrix& InComponentToWorld, const FString& InPackageName, TArray<FRawMeshTracker>& OutRawMeshTrackers, TArray<FRawMesh>& OutRawMeshes, TArray<UMaterialInterface*>& OutMaterials)
{
	const int32 BaseMaterialIndex = OutMaterials.Num();

	// Export all LODs to raw meshes
	const int32 NumLODs = InSkinnedMeshComponent->GetNumLODs();
	//The cpu skinned vertice is not valid under min lod
	int32 MinLOD = InSkinnedMeshComponent->ComputeMinLOD();

	for (int32 OverallLODIndex = MinLOD; OverallLODIndex < InOverallMaxLODs; OverallLODIndex++)
	{
		int32 LODIndexRead = FMath::Min(OverallLODIndex, NumLODs - 1);

		FRawMesh& RawMesh = OutRawMeshes[OverallLODIndex];
		FRawMeshTracker& RawMeshTracker = OutRawMeshTrackers[OverallLODIndex];
		const int32 BaseVertexIndex = RawMesh.VertexPositions.Num();

		FSkeletalMeshLODInfo& SrcLODInfo = *(InSkinnedMeshComponent->GetSkinnedAsset()->GetLODInfo(LODIndexRead));

		// Get the CPU skinned verts for this LOD
		TArray<FFinalSkinVertex> FinalVertices;
		InSkinnedMeshComponent->GetCPUSkinnedVertices(FinalVertices, LODIndexRead);

		FSkeletalMeshRenderData& SkeletalMeshRenderData = InSkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData();
		FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LODIndexRead];

		// Copy skinned vertex positions
		for (int32 VertIndex = 0; VertIndex < FinalVertices.Num(); ++VertIndex)
		{
			RawMesh.VertexPositions.Add((FVector4f)InComponentToWorld.TransformPosition((FVector)FinalVertices[VertIndex].Position));
		}

		const uint32 NumTexCoords = FMath::Min(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords(), (uint32)MAX_MESH_TEXTURE_COORDS);
		const int32 NumSections = LODData.RenderSections.Num();
		FRawStaticIndexBuffer16or32Interface& IndexBuffer = *LODData.MultiSizeIndexContainer.GetIndexBuffer();

		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			const FSkelMeshRenderSection& SkelMeshSection = LODData.RenderSections[SectionIndex];
			if (InSkinnedMeshComponent->IsMaterialSectionShown(SkelMeshSection.MaterialIndex, LODIndexRead))
			{
				// Build 'wedge' info
				const int32 NumWedges = SkelMeshSection.NumTriangles * 3;
				for(int32 WedgeIndex = 0; WedgeIndex < NumWedges; WedgeIndex++)
				{
					const int32 VertexIndexForWedge = IndexBuffer.Get(SkelMeshSection.BaseIndex + WedgeIndex);

					RawMesh.WedgeIndices.Add(BaseVertexIndex + VertexIndexForWedge);

					const FFinalSkinVertex& SkinnedVertex = FinalVertices[VertexIndexForWedge];
					const FVector3f TangentX = (FVector4f)InComponentToWorld.TransformVector(SkinnedVertex.TangentX.ToFVector());
					const FVector3f TangentZ = (FVector4f)InComponentToWorld.TransformVector(SkinnedVertex.TangentZ.ToFVector());
					const FVector4 UnpackedTangentZ = SkinnedVertex.TangentZ.ToFVector4();
					const FVector3f TangentY = (TangentZ ^ TangentX).GetSafeNormal() * UnpackedTangentZ.W;

					RawMesh.WedgeTangentX.Add(TangentX);
					RawMesh.WedgeTangentY.Add(TangentY);
					RawMesh.WedgeTangentZ.Add(TangentZ);

					for (uint32 TexCoordIndex = 0; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; TexCoordIndex++)
					{
						if (TexCoordIndex >= NumTexCoords)
						{
							RawMesh.WedgeTexCoords[TexCoordIndex].AddDefaulted();
						}
						else
						{
							RawMesh.WedgeTexCoords[TexCoordIndex].Add(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndexForWedge, TexCoordIndex));
							RawMeshTracker.bValidTexCoords[TexCoordIndex] = true;
						}
					}

					if (LODData.StaticVertexBuffers.ColorVertexBuffer.IsInitialized() && LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
					{
						RawMesh.WedgeColors.Add(LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndexForWedge));
						RawMeshTracker.bValidColors = true;
					}
					else
					{
						RawMesh.WedgeColors.Add(FColor::White);
					}
				}

				int32 MaterialIndex = SkelMeshSection.MaterialIndex;
				// use the remapping of material indices if there is a valid value
				if (SrcLODInfo.LODMaterialMap.IsValidIndex(SectionIndex) && SrcLODInfo.LODMaterialMap[SectionIndex] != INDEX_NONE)
				{
					MaterialIndex = FMath::Clamp<int32>(SrcLODInfo.LODMaterialMap[SectionIndex], 0, InSkinnedMeshComponent->GetSkinnedAsset()->GetMaterials().Num() - 1);
				}

				// copy face info
				for (uint32 TriIndex = 0; TriIndex < SkelMeshSection.NumTriangles; TriIndex++)
				{
					RawMesh.FaceMaterialIndices.Add(BaseMaterialIndex + MaterialIndex);
					RawMesh.FaceSmoothingMasks.Add(0); // Assume this is ignored as bRecomputeNormals is false
				}
			}
		}

		RawMesh.CompactMaterialIndices();
	}

	ProcessMaterials<USkinnedMeshComponent>(InSkinnedMeshComponent, InPackageName, OutMaterials);
}

// Helper function for ConvertMeshesToStaticMesh
static bool IsValidStaticMeshComponent(UStaticMeshComponent* InComponent)
{
	return InComponent && InComponent->GetStaticMesh() && InComponent->GetStaticMesh()->GetRenderData() && InComponent->IsVisible();
}

// Helper function for ConvertMeshesToStaticMesh
static void StaticMeshToRawMeshes(UStaticMeshComponent* InStaticMeshComponent, int32 InOverallMaxLODs, const FMatrix& InComponentToWorld, const FString& InPackageName, TArray<FRawMeshTracker>& OutRawMeshTrackers, TArray<FRawMesh>& OutRawMeshes, TArray<UMaterialInterface*>& OutMaterials)
{
	const int32 BaseMaterialIndex = OutMaterials.Num();

	const int32 NumLODs = InStaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources.Num();

	const UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(InStaticMeshComponent);
	const int32 NumInstances = ISMC ? ISMC->GetInstanceCount() : 1;
	auto GetInstanceTransformMatrix = [ISMC, InStaticMeshComponent, NumInstances](int32 InstanceIndex)
		{
			if (ISMC && InstanceIndex <= NumInstances)
			{
				FTransform InstanceTransform;
				ISMC->GetInstanceTransform(InstanceIndex, InstanceTransform);
				return InstanceTransform.ToMatrixWithScale();
			}

			return FMatrix::Identity;
		};

	for (int32 OverallLODIndex = 0; OverallLODIndex < InOverallMaxLODs; OverallLODIndex++)
	{
		int32 LODIndexRead = FMath::Min(OverallLODIndex, NumLODs - 1);

		FRawMesh& RawMesh = OutRawMeshes[OverallLODIndex];
		FRawMeshTracker& RawMeshTracker = OutRawMeshTrackers[OverallLODIndex];
		const FStaticMeshLODResources& LODResource = InStaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndexRead];

		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; InstanceIndex++)
		{
			const FMatrix InstanceToWorld = GetInstanceTransformMatrix(InstanceIndex) * InComponentToWorld;
			const int32 BaseVertexIndex = RawMesh.VertexPositions.Num();

			for (int32 VertIndex = 0; VertIndex < LODResource.GetNumVertices(); ++VertIndex)
			{
				RawMesh.VertexPositions.Add(FVector4f(InstanceToWorld.TransformPosition((FVector)LODResource.VertexBuffers.PositionVertexBuffer.VertexPosition((uint32)VertIndex))));
			}

			const FIndexArrayView IndexArrayView = LODResource.IndexBuffer.GetArrayView();
			const FStaticMeshVertexBuffer& StaticMeshVertexBuffer = LODResource.VertexBuffers.StaticMeshVertexBuffer;
			const int32 NumTexCoords = FMath::Min(StaticMeshVertexBuffer.GetNumTexCoords(), (uint32)MAX_MESH_TEXTURE_COORDS);
			const int32 NumSections = LODResource.Sections.Num();

			for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
			{
				const FStaticMeshSection& StaticMeshSection = LODResource.Sections[SectionIndex];

				const int32 NumIndices = StaticMeshSection.NumTriangles * 3;
				for (int32 IndexIndex = 0; IndexIndex < NumIndices; IndexIndex++)
				{
					int32 Index = IndexArrayView[StaticMeshSection.FirstIndex + IndexIndex];
					RawMesh.WedgeIndices.Add(BaseVertexIndex + Index);

					RawMesh.WedgeTangentX.Add(FVector4f(InstanceToWorld.TransformVector(FVector(StaticMeshVertexBuffer.VertexTangentX(Index)))));
					RawMesh.WedgeTangentY.Add(FVector4f(InstanceToWorld.TransformVector(FVector(StaticMeshVertexBuffer.VertexTangentY(Index)))));
					RawMesh.WedgeTangentZ.Add(FVector4f(InstanceToWorld.TransformVector(FVector(StaticMeshVertexBuffer.VertexTangentZ(Index)))));

					for (int32 TexCoordIndex = 0; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; TexCoordIndex++)
					{
						if (TexCoordIndex >= NumTexCoords)
						{
							RawMesh.WedgeTexCoords[TexCoordIndex].AddDefaulted();
						}
						else
						{
							RawMesh.WedgeTexCoords[TexCoordIndex].Add(StaticMeshVertexBuffer.GetVertexUV(Index, TexCoordIndex));
							RawMeshTracker.bValidTexCoords[TexCoordIndex] = true;
						}
					}

					if (LODResource.VertexBuffers.ColorVertexBuffer.IsInitialized())
					{
						RawMesh.WedgeColors.Add(LODResource.VertexBuffers.ColorVertexBuffer.VertexColor(Index));
						RawMeshTracker.bValidColors = true;
					}
					else
					{
						RawMesh.WedgeColors.Add(FColor::White);
					}
				}

				// copy face info
				for (uint32 TriIndex = 0; TriIndex < StaticMeshSection.NumTriangles; TriIndex++)
				{
					RawMesh.FaceMaterialIndices.Add(BaseMaterialIndex + StaticMeshSection.MaterialIndex);
					RawMesh.FaceSmoothingMasks.Add(0); // Assume this is ignored as bRecomputeNormals is false
				}
			}
		}
	}

	ProcessMaterials<UStaticMeshComponent>(InStaticMeshComponent, InPackageName, OutMaterials);
}

UStaticMesh* FMeshUtilities::ConvertMeshesToStaticMesh(const TArray<UMeshComponent*>& InMeshComponents, const FTransform& InRootTransform, const FString& InPackageName)
{
	UStaticMesh* StaticMesh = nullptr;

	// Build a package name to use
	FString MeshName;
	FString PackageName;
	if (InPackageName.IsEmpty())
	{
		FString NewNameSuggestion = FString(TEXT("StaticMesh"));
		FString DefaultPath;
		const FString DefaultDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		FPackageName::TryConvertFilenameToLongPackageName(DefaultDirectory, DefaultPath);

		if (DefaultPath.IsEmpty())
		{
			DefaultPath = TEXT("/Game/Meshes");
		}

		FString PackageNameSuggestion = DefaultPath / NewNameSuggestion;
		FString Name;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(PackageNameSuggestion, TEXT(""), PackageNameSuggestion, Name);

		TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("ConvertToStaticMeshPickName", "Choose New StaticMesh Location"))
			.DefaultAssetPath(FText::FromString(PackageNameSuggestion));

		if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
		{
			// Get the full name of where we want to create the mesh asset.
			PackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
			MeshName = FPackageName::GetLongPackageAssetName(PackageName);

			// Check if the user inputed a valid asset name, if they did not, give it the generated default name
			if (MeshName.IsEmpty())
			{
				// Use the defaults that were already generated.
				PackageName = PackageNameSuggestion;
				MeshName = *Name;
			}
		}
	}
	else
	{
		PackageName = InPackageName;
		MeshName = *FPackageName::GetLongPackageAssetName(PackageName);
	}

	if(!PackageName.IsEmpty() && !MeshName.IsEmpty())
	{
		TArray<FRawMesh> RawMeshes;
		TArray<UMaterialInterface*> Materials;

		TArray<FRawMeshTracker> RawMeshTrackers;

		FMatrix WorldToRoot = InRootTransform.ToMatrixWithScale().Inverse();

		// first do a pass to determine the max LOD level we will be combining meshes into
		int32 OverallMaxLODs = 0;
		for (UMeshComponent* MeshComponent : InMeshComponents)
		{
			USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent);
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);

			if (IsValidSkinnedMeshComponent(SkinnedMeshComponent))
			{
				OverallMaxLODs = FMath::Max(SkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData().LODRenderData.Num(), OverallMaxLODs);
			}
			else if(IsValidStaticMeshComponent(StaticMeshComponent))
			{
				OverallMaxLODs = FMath::Max(StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources.Num(), OverallMaxLODs);
			}
		}
		
		// Resize raw meshes to accommodate the number of LODs we will need
		RawMeshes.SetNum(OverallMaxLODs);
		RawMeshTrackers.SetNum(OverallMaxLODs);

		// Export all visible components
		for (UMeshComponent* MeshComponent : InMeshComponents)
		{
			FMatrix ComponentToWorld = MeshComponent->GetComponentTransform().ToMatrixWithScale() * WorldToRoot;

			USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent);
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);

			if (IsValidSkinnedMeshComponent(SkinnedMeshComponent))
			{
				SkinnedMeshToRawMeshes(SkinnedMeshComponent, OverallMaxLODs, ComponentToWorld, PackageName, RawMeshTrackers, RawMeshes, Materials);
			}
			else if (IsValidStaticMeshComponent(StaticMeshComponent))
			{
				StaticMeshToRawMeshes(StaticMeshComponent, OverallMaxLODs, ComponentToWorld, PackageName, RawMeshTrackers, RawMeshes, Materials);
			}
		}

		uint32 MaxInUseTextureCoordinate = 0;

		// scrub invalid vert color & tex coord data
		check(RawMeshes.Num() == RawMeshTrackers.Num());
		for (int32 RawMeshIndex = 0; RawMeshIndex < RawMeshes.Num(); RawMeshIndex++)
		{
			if (!RawMeshTrackers[RawMeshIndex].bValidColors)
			{
				RawMeshes[RawMeshIndex].WedgeColors.Empty();
			}

			for (uint32 TexCoordIndex = 0; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; TexCoordIndex++)
			{
				if (!RawMeshTrackers[RawMeshIndex].bValidTexCoords[TexCoordIndex])
				{
					RawMeshes[RawMeshIndex].WedgeTexCoords[TexCoordIndex].Empty();
				}
				else
				{
					// Store first texture coordinate index not in use
					MaxInUseTextureCoordinate = FMath::Max(MaxInUseTextureCoordinate, TexCoordIndex);
				}
			}
		}

		// Check if we got some valid data.
		bool bValidData = false;
		for (FRawMesh& RawMesh : RawMeshes)
		{
			if (RawMesh.IsValidOrFixable())
			{
				bValidData = true;
				break;
			}
		}

		if (bValidData)
		{
			// Then find/create it.
			UPackage* Package = CreatePackage( *PackageName);
			check(Package);

			// Create StaticMesh object
			StaticMesh = NewObject<UStaticMesh>(Package, *MeshName, RF_Public | RF_Standalone);
			StaticMesh->InitResources();

			StaticMesh->SetLightingGuid();

			// Determine which texture coordinate map should be used for storing/generating the lightmap UVs
			const uint32 LightMapIndex = FMath::Min(MaxInUseTextureCoordinate + 1, (uint32)MAX_MESH_TEXTURE_COORDS - 1);

			// Add source to new StaticMesh
			for (FRawMesh& RawMesh : RawMeshes)
			{
				if (RawMesh.IsValidOrFixable())
				{
					FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();
					SrcModel.BuildSettings.bRecomputeNormals = false;
					SrcModel.BuildSettings.bRecomputeTangents = false;
					SrcModel.BuildSettings.bRemoveDegenerates = true;
					SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
					SrcModel.BuildSettings.bUseFullPrecisionUVs = false;
					SrcModel.BuildSettings.bGenerateLightmapUVs = true;
					SrcModel.BuildSettings.SrcLightmapIndex = 0;
					SrcModel.BuildSettings.DstLightmapIndex = LightMapIndex;
					SrcModel.SaveRawMesh(RawMesh);
				}
			}

			// Copy materials to new mesh 
			for(UMaterialInterface* Material : Materials)
			{
				StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material));
			}
			
			//Set the Imported version before calling the build
			StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

			// Set light map coordinate index to match DstLightmapIndex
			StaticMesh->SetLightMapCoordinateIndex(LightMapIndex);

			// setup section info map
			for (int32 RawMeshLODIndex = 0; RawMeshLODIndex < RawMeshes.Num(); RawMeshLODIndex++)
			{
				const FRawMesh& RawMesh = RawMeshes[RawMeshLODIndex];
				TArray<int32> UniqueMaterialIndices;
				for (int32 MaterialIndex : RawMesh.FaceMaterialIndices)
				{
					UniqueMaterialIndices.AddUnique(MaterialIndex);
				}

				int32 SectionIndex = 0;
				for (int32 UniqueMaterialIndex : UniqueMaterialIndices)
				{
					if (RawMesh.MaterialIndexToImportIndex.IsValidIndex(UniqueMaterialIndex))
					{
						StaticMesh->GetSectionInfoMap().Set(RawMeshLODIndex, SectionIndex, FMeshSectionInfo(RawMesh.MaterialIndexToImportIndex[UniqueMaterialIndex]));
					}
					else
					{
						StaticMesh->GetSectionInfoMap().Set(RawMeshLODIndex, SectionIndex, FMeshSectionInfo(UniqueMaterialIndex));
					}
					SectionIndex++;
				}
			}
			StaticMesh->GetOriginalSectionInfoMap().CopyFrom(StaticMesh->GetSectionInfoMap());

			// Build mesh from source
			StaticMesh->Build(false);
			StaticMesh->PostEditChange();

			StaticMesh->MarkPackageDirty();

			// Notify asset registry of new asset
			FAssetRegistryModule::AssetCreated(StaticMesh);

			// Display notification so users can quickly access the mesh
			if (GIsEditor)
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("SkeletalMeshConverted", "Successfully Converted Mesh"), FText::FromString(StaticMesh->GetName())));
				Info.ExpireDuration = 8.0f;
				Info.bUseLargeFont = false;
				Info.Hyperlink = FSimpleDelegate::CreateLambda([=]() { GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(TArray<UObject*>({ StaticMesh })); });
				Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewAnimationHyperlink", "Open {0}"), FText::FromString(StaticMesh->GetName()));
				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if ( Notification.IsValid() )
				{
					Notification->SetCompletionState( SNotificationItem::CS_Success );
				}
			}
		}
	}

	return StaticMesh;
}

/**
* Builds a renderable skeletal mesh LOD model. Note that the array of chunks
* will be destroyed during this process!
* @param LODModel				Upon return contains a renderable skeletal mesh LOD model.
* @param RefSkeleton			The reference skeleton associated with the model.
* @param Chunks					Skinned mesh chunks from which to build the renderable model.
* @param PointToOriginalMap		Maps a vertex's RawPointIdx to its index at import time.
* @param bCacheOptimize			Whether or not to cache optimize the index buffer in each section.
*/
void FMeshUtilities::BuildSkeletalModelFromChunks(
	FSkeletalMeshLODModel& LODModel,
	const FReferenceSkeleton& RefSkeleton,
	TArray<FSkinnedMeshChunk*>& Chunks,
	const TArray<int32>& PointToOriginalMap,
	bool bCacheOptimize
)
{
#if WITH_EDITORONLY_DATA
	// Clear out any data currently held in the LOD model.
	LODModel.Sections.Empty();
	LODModel.NumVertices = 0;
	LODModel.IndexBuffer.Empty();

	// Setup the section and chunk arrays on the model.
	for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		FSkinnedMeshChunk* SrcChunk = Chunks[ChunkIndex];

		FSkelMeshSection& Section = *new(LODModel.Sections) FSkelMeshSection();
		Section.MaterialIndex = SrcChunk->MaterialIndex;
		Exchange(Section.BoneMap, SrcChunk->BoneMap);

		Section.OriginalDataSectionIndex = SrcChunk->OriginalSectionIndex;
		Section.ChunkedParentSectionIndex = SrcChunk->ParentChunkSectionIndex;

		// Update the active bone indices on the LOD model.
		for (int32 BoneIndex = 0; BoneIndex < Section.BoneMap.Num(); ++BoneIndex)
		{
			LODModel.ActiveBoneIndices.AddUnique(Section.BoneMap[BoneIndex]);
		}
	}

	// ensure parent exists with incoming active bone indices, and the result should be sorted	
	RefSkeleton.EnsureParentsExistAndSort(LODModel.ActiveBoneIndices);

	// Reset 'final vertex to import vertex' map info
	LODModel.MeshToImportVertexMap.Empty();
	LODModel.MaxImportVertex = 0;

	// Keep track of index mapping to chunk vertex offsets
	TArray< TArray<uint32> > VertexIndexRemap;
	VertexIndexRemap.Empty(LODModel.Sections.Num());
	TArray<uint32>& RawPointIndices = LODModel.GetRawPointIndices();
	RawPointIndices.Reset();
	// Pack the chunk vertices into a single vertex buffer.
	LODModel.NumVertices = 0;

	int32 PrevMaterialIndex = -1;
	int32 CurrentChunkBaseVertexIndex = -1; 	// base vertex index for all chunks of the same material
	int32 CurrentChunkVertexCount = -1; 		// total vertex count for all chunks of the same material
	int32 CurrentVertexIndex = 0; 			// current vertex index added to the index buffer for all chunks of the same material

	// rearrange the vert order to minimize the data fetched by the GPU
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		if (IsInGameThread())
		{
			GWarn->StatusUpdate(SectionIndex, LODModel.Sections.Num(), NSLOCTEXT("UnrealEd", "ProcessingSections", "Processing Sections"));
		}

		FSkinnedMeshChunk* SrcChunk = Chunks[SectionIndex];
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		TArray<FSoftSkinBuildVertex>& ChunkVertices = SrcChunk->Vertices;
		TArray<uint32>& ChunkIndices = SrcChunk->Indices;

		// Reorder the section index buffer for better vertex cache efficiency.
		if (bCacheOptimize)
		{
			CacheOptimizeIndexBuffer(ChunkIndices);
		}

		// Calculate the number of triangles in the section.  Note that CacheOptimize may change the number of triangles in the index buffer!
		Section.NumTriangles = ChunkIndices.Num() / 3;
		TArray<FSoftSkinBuildVertex> OriginalVertices;
		Exchange(ChunkVertices, OriginalVertices);
		ChunkVertices.AddUninitialized(OriginalVertices.Num());

		TArray<int32> IndexCache;
		IndexCache.AddUninitialized(ChunkVertices.Num());
		FMemory::Memset(IndexCache.GetData(), INDEX_NONE, IndexCache.Num() * IndexCache.GetTypeSize());
		int32 NextAvailableIndex = 0;
		// Go through the indices and assign them new values that are coherent where possible
		for (int32 Index = 0; Index < ChunkIndices.Num(); Index++)
		{
			const int32 OriginalIndex = ChunkIndices[Index];
			const int32 CachedIndex = IndexCache[OriginalIndex];

			if (CachedIndex == INDEX_NONE)
			{
				// No new index has been allocated for this existing index, assign a new one
				ChunkIndices[Index] = NextAvailableIndex;
				// Mark what this index has been assigned to
				IndexCache[OriginalIndex] = NextAvailableIndex;
				NextAvailableIndex++;
			}
			else
			{
				// Reuse an existing index assignment
				ChunkIndices[Index] = CachedIndex;
			}
			// Reorder the vertices based on the new index assignment
			ChunkVertices[ChunkIndices[Index]] = OriginalVertices[OriginalIndex];
		}
	}

	// Build the arrays of rigid and soft vertices on the model's chunks.
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		TArray<FSoftSkinBuildVertex>& ChunkVertices = Chunks[SectionIndex]->Vertices;

		if (IsInGameThread())
		{
			// Only update status if in the game thread.  When importing morph targets, this function can run in another thread
			GWarn->StatusUpdate(SectionIndex, LODModel.Sections.Num(), NSLOCTEXT("UnrealEd", "ProcessingChunks", "Processing Chunks"));
		}

		CurrentVertexIndex = 0;
		CurrentChunkVertexCount = 0;
		PrevMaterialIndex = Section.MaterialIndex;

		// Calculate the offset to this chunk's vertices in the vertex buffer.
		Section.BaseVertexIndex = CurrentChunkBaseVertexIndex = LODModel.NumVertices;

		// Update the size of the vertex buffer.
		LODModel.NumVertices += ChunkVertices.Num();

		// Separate the section's vertices into rigid and soft vertices.
		TArray<uint32>& ChunkVertexIndexRemap = *new(VertexIndexRemap)TArray<uint32>();
		ChunkVertexIndexRemap.AddUninitialized(ChunkVertices.Num());

		for (int32 VertexIndex = 0; VertexIndex < ChunkVertices.Num(); VertexIndex++)
		{
			const FSoftSkinBuildVertex& SoftVertex = ChunkVertices[VertexIndex];

			FSoftSkinVertex NewVertex;
			NewVertex.Position = SoftVertex.Position;
			NewVertex.TangentX = SoftVertex.TangentX;
			NewVertex.TangentY = SoftVertex.TangentY;
			NewVertex.TangentZ = SoftVertex.TangentZ;
			FMemory::Memcpy(NewVertex.UVs, SoftVertex.UVs, sizeof(FVector2f)*MAX_TEXCOORDS);
			NewVertex.Color = SoftVertex.Color;
			for (int32 i = 0; i < MAX_TOTAL_INFLUENCES; ++i)
			{
				// it only adds to the bone map if it has weight on it
				// BoneMap contains only the bones that has influence with weight of >0.f
				// so here, just make sure it is included before setting the data
				if (Section.BoneMap.IsValidIndex(SoftVertex.InfluenceBones[i]))
				{
					NewVertex.InfluenceBones[i] = SoftVertex.InfluenceBones[i];
					NewVertex.InfluenceWeights[i] = SoftVertex.InfluenceWeights[i];
				}
			}
			Section.SoftVertices.Add(NewVertex);
			ChunkVertexIndexRemap[VertexIndex] = (uint32)(Section.BaseVertexIndex + CurrentVertexIndex);
			CurrentVertexIndex++;
			// add the index to the original wedge point source of this vertex
			RawPointIndices.Add(SoftVertex.PointWedgeIdx);
			// Also remember import index
			const int32 RawVertIndex = PointToOriginalMap[SoftVertex.PointWedgeIdx];
			LODModel.MeshToImportVertexMap.Add(RawVertIndex);
			LODModel.MaxImportVertex = FMath::Max<float>(LODModel.MaxImportVertex, RawVertIndex);
		}

		// update NumVertices
		Section.NumVertices = Section.SoftVertices.Num();

		// update max bone influences
		Section.CalcMaxBoneInfluences();
		Section.CalcUse16BitBoneIndex();

		// Log info about the chunk.
		UE_LOGF(LogSkeletalMesh, Verbose, "Section %u: %u vertices, %u active bones",
			SectionIndex,
			Section.GetNumVertices(),
			Section.BoneMap.Num()
			);
	}

	// Finish building the sections.
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

		const TArray<uint32>& SectionIndices = Chunks[SectionIndex]->Indices;

		Section.BaseIndex = LODModel.IndexBuffer.Num();
		const int32 NumIndices = SectionIndices.Num();
		const TArray<uint32>& SectionVertexIndexRemap = VertexIndexRemap[SectionIndex];
		for (int32 Index = 0; Index < NumIndices; Index++)
		{
			uint32 VertexIndex = SectionVertexIndexRemap[SectionIndices[Index]];
			LODModel.IndexBuffer.Add(VertexIndex);
		}
	}

	// Free the skinned mesh chunks which are no longer needed.
	for (int32 i = 0; i < Chunks.Num(); ++i)
	{
		delete Chunks[i];
		Chunks[i] = nullptr;
	}
	Chunks.Empty();

	// Compute the required bones for this model.
	USkeletalMesh::CalculateRequiredBones(LODModel, RefSkeleton, nullptr);

#endif // #if WITH_EDITORONLY_DATA
}

/*------------------------------------------------------------------------------
Common functionality.
------------------------------------------------------------------------------*/

static int32 ComputeNumTexCoords(FRawMesh const& RawMesh, int32 MaxSupportedTexCoords)
{
	int32 NumWedges = RawMesh.WedgeIndices.Num();
	int32 NumTexCoords = 0;
	for (int32 TexCoordIndex = 0; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; ++TexCoordIndex)
	{
		if (RawMesh.WedgeTexCoords[TexCoordIndex].Num() != NumWedges)
		{
			break;
		}
		NumTexCoords++;
	}
	return FMath::Min(NumTexCoords, MaxSupportedTexCoords);
}

static inline FVector3f GetPositionForWedge(FRawMesh const& Mesh, int32 WedgeIndex)
{
	int32 VertexIndex = Mesh.WedgeIndices[WedgeIndex];
	return Mesh.VertexPositions[VertexIndex];
}

struct FMeshEdgeDef
{
	int32	Vertices[2];
	int32	Faces[2];
};

/**
* This helper class builds the edge list for a mesh. It uses a hash of vertex
* positions to edges sharing that vertex to remove the n^2 searching of all
* previously added edges. This class is templatized so it can be used with
* either static mesh or skeletal mesh vertices
*/
template <class VertexClass> class TEdgeBuilder
{
protected:
	/**
	* The list of indices to build the edge data from
	*/
	const TArray<uint32>& Indices;
	/**
	* The array of verts for vertex position comparison
	*/
	const TArray<VertexClass>& Vertices;
	/**
	* The array of edges to create
	*/
	TArray<FMeshEdgeDef>& Edges;
	/**
	* List of edges that start with a given vertex
	*/
	TMultiMap<FVector3f, FMeshEdgeDef*> VertexToEdgeList;

	/**
	* This function determines whether a given edge matches or not. It must be
	* provided by derived classes since they have the specific information that
	* this class doesn't know about (vertex info, influences, etc)
	*
	* @param Index1 The first index of the edge being checked
	* @param Index2 The second index of the edge
	* @param OtherEdge The edge to compare. Was found via the map
	*
	* @return true if the edge is a match, false otherwise
	*/
	virtual bool DoesEdgeMatch(int32 Index1, int32 Index2, FMeshEdgeDef* OtherEdge) = 0;

	/**
	* Searches the list of edges to see if this one matches an existing and
	* returns a pointer to it if it does
	*
	* @param Index1 the first index to check for
	* @param Index2 the second index to check for
	*
	* @return NULL if no edge was found, otherwise the edge that was found
	*/
	inline FMeshEdgeDef* FindOppositeEdge(int32 Index1, int32 Index2)
	{
		FMeshEdgeDef* Edge = NULL;
		// Search the hash for a corresponding vertex
		WorkingEdgeList.Reset();
		VertexToEdgeList.MultiFind(Vertices[Index2].Position, WorkingEdgeList);
		// Now search through the array for a match or not
		for (int32 EdgeIndex = 0; EdgeIndex < WorkingEdgeList.Num() && Edge == NULL;
			EdgeIndex++)
		{
			FMeshEdgeDef* OtherEdge = WorkingEdgeList[EdgeIndex];
			// See if this edge matches the passed in edge
			if (OtherEdge != NULL && DoesEdgeMatch(Index1, Index2, OtherEdge))
			{
				// We have a match
				Edge = OtherEdge;
			}
		}
		return Edge;
	}

	/**
	* Updates an existing edge if found or adds the new edge to the list
	*
	* @param Index1 the first index in the edge
	* @param Index2 the second index in the edge
	* @param Triangle the triangle that this edge was found in
	*/
	inline void AddEdge(int32 Index1, int32 Index2, int32 Triangle)
	{
		// If this edge matches another then just fill the other triangle
		// otherwise add it
		FMeshEdgeDef* OtherEdge = FindOppositeEdge(Index1, Index2);
		if (OtherEdge == NULL)
		{
			// Add a new edge to the array
			int32 EdgeIndex = Edges.AddZeroed();
			Edges[EdgeIndex].Vertices[0] = Index1;
			Edges[EdgeIndex].Vertices[1] = Index2;
			Edges[EdgeIndex].Faces[0] = Triangle;
			Edges[EdgeIndex].Faces[1] = -1;
			// Also add this edge to the hash for faster searches
			// NOTE: This relies on the array never being realloced!
			VertexToEdgeList.Add(Vertices[Index1].Position, &Edges[EdgeIndex]);
		}
		else
		{
			OtherEdge->Faces[1] = Triangle;
		}
	}

public:
	/**
	* Initializes the values for the code that will build the mesh edge list
	*/
	TEdgeBuilder(const TArray<uint32>& InIndices,
		const TArray<VertexClass>& InVertices,
		TArray<FMeshEdgeDef>& OutEdges) :
		Indices(InIndices), Vertices(InVertices), Edges(OutEdges)
	{
		// Presize the array so that there are no extra copies being done
		// when adding edges to it
		Edges.Empty(Indices.Num());
	}

	/**
	* Virtual dtor
	*/
	virtual ~TEdgeBuilder(){}


	/**
	* Uses a hash of indices to edge lists so that it can avoid the n^2 search
	* through the full edge list
	*/
	void FindEdges(void)
	{
		// @todo Handle something other than trilists when building edges
		int32 TriangleCount = Indices.Num() / 3;
		int32 EdgeCount = 0;
		// Work through all triangles building the edges
		for (int32 Triangle = 0; Triangle < TriangleCount; Triangle++)
		{
			// Determine the starting index
			int32 TriangleIndex = Triangle * 3;
			// Get the indices for the triangle
			int32 Index1 = Indices[TriangleIndex];
			int32 Index2 = Indices[TriangleIndex + 1];
			int32 Index3 = Indices[TriangleIndex + 2];
			// Add the first to second edge
			AddEdge(Index1, Index2, Triangle);
			// Now add the second to third
			AddEdge(Index2, Index3, Triangle);
			// Add the third to first edge
			AddEdge(Index3, Index1, Triangle);
		}
	}

private:
	TArray<FMeshEdgeDef*> WorkingEdgeList;
};

/**
* This is the static mesh specific version for finding edges
*/
class FStaticMeshEdgeBuilder : public TEdgeBuilder<FStaticMeshBuildVertex>
{
public:
	/**
	* Constructor that passes all work to the parent class
	*/
	FStaticMeshEdgeBuilder(const TArray<uint32>& InIndices,
		const TArray<FStaticMeshBuildVertex>& InVertices,
		TArray<FMeshEdgeDef>& OutEdges) :
		TEdgeBuilder<FStaticMeshBuildVertex>(InIndices, InVertices, OutEdges)
	{
	}

	/**
	* This function determines whether a given edge matches or not for a static mesh
	*
	* @param Index1 The first index of the edge being checked
	* @param Index2 The second index of the edge
	* @param OtherEdge The edge to compare. Was found via the map
	*
	* @return true if the edge is a match, false otherwise
	*/
	bool DoesEdgeMatch(int32 Index1, int32 Index2, FMeshEdgeDef* OtherEdge)
	{
		return Vertices[OtherEdge->Vertices[1]].Position == Vertices[Index1].Position &&
			OtherEdge->Faces[1] == -1;
	}
};

static void ComputeTriangleTangents(
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	const TArray<FVector2f>& InUVs,
	TArray<FVector3f>& OutTangentX,
	TArray<FVector3f>& OutTangentY,
	TArray<FVector3f>& OutTangentZ,
	float ComparisonThreshold
	)
{
	const int32 NumTriangles = InIndices.Num() / 3;
	OutTangentX.Empty(NumTriangles);
	OutTangentY.Empty(NumTriangles);
	OutTangentZ.Empty(NumTriangles);

	//Currently GetSafeNormal do not support 0.0f threshold properly
	float RealComparisonThreshold = FMath::Max(ComparisonThreshold, FLT_MIN);
	
	TArray<FVector3f> TriangleTangents;
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		FVector3f Positions[3];
		FVector2f UVs[3];
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Positions[Index] = InVertices[InIndices[TriangleIndex * 3 + Index]];
			UVs[Index] = InUVs[TriangleIndex * 3 + Index];
		}
		TriangleTangents.Reset();
		CalculateTriangleTangentInternal(Positions[0], UVs[0], Positions[1], UVs[1], Positions[2], UVs[2], TriangleTangents, RealComparisonThreshold);
		check(TriangleTangents.Num() == 3);
		OutTangentX.Add(TriangleTangents[0]);
		OutTangentY.Add(TriangleTangents[1]);
		OutTangentZ.Add(TriangleTangents[2]);
	}

	check(OutTangentX.Num() == NumTriangles);
	check(OutTangentY.Num() == NumTriangles);
	check(OutTangentZ.Num() == NumTriangles);
}

static void ComputeTriangleTangents(
	TArray<FVector3f>& OutTangentX,
	TArray<FVector3f>& OutTangentY,
	TArray<FVector3f>& OutTangentZ,
	FRawMesh const& RawMesh,
	float ComparisonThreshold
	)
{
	ComputeTriangleTangents(RawMesh.VertexPositions, RawMesh.WedgeIndices, RawMesh.WedgeTexCoords[0], OutTangentX, OutTangentY, OutTangentZ, ComparisonThreshold);
}

/**
* Create a table that maps the corner of each face to its overlapping corners.
* @param OutOverlappingCorners - Maps a corner index to the indices of all overlapping corners.
* @param InVertices - Triangle vertex positions for the mesh for which to compute overlapping corners.
* @param InIndices - Triangle indices for the mesh for which to compute overlapping corners.
* @param ComparisonThreshold - Positions are considered equal if all absolute differences between their X, Y and Z coordinates are less or equal to this value.
*/
void FMeshUtilities::FindOverlappingCorners(
	FOverlappingCorners& OutOverlappingCorners,
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	float ComparisonThreshold) const
{
	OutOverlappingCorners = FOverlappingCorners(InVertices, InIndices, ComparisonThreshold);
}

/**
* Create a table that maps the corner of each face to its overlapping corners.
* @param OutOverlappingCorners - Maps a corner index to the indices of all overlapping corners.
* @param RawMesh - The mesh for which to compute overlapping corners.
* @param ComparisonThreshold - Positions are considered equal if all absolute differences between their X, Y and Z coordinates are less or equal to this value.
*/
void FMeshUtilities::FindOverlappingCorners(
	FOverlappingCorners& OutOverlappingCorners,
	FRawMesh const& RawMesh,
	float ComparisonThreshold
	) const
{
	OutOverlappingCorners = FOverlappingCorners(RawMesh.VertexPositions, RawMesh.WedgeIndices, ComparisonThreshold);
}

/**
* Smoothing group interpretation helper structure.
*/
struct FFanFace
{
	int32 FaceIndex;
	int32 LinkedVertexIndex;
	bool bFilled;
	bool bBlendTangents;
	bool bBlendNormals;
};

static void ComputeTangents(
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	const TArray<FVector2f>& InUVs,
	const TArray<uint32>& SmoothingGroupIndices,
	const FOverlappingCorners& OverlappingCorners,
	TArray<FVector3f>& OutTangentX,
	TArray<FVector3f>& OutTangentY,
	TArray<FVector3f>& OutTangentZ,
	const uint32 TangentOptions
	)
{
	bool bBlendOverlappingNormals = (TangentOptions & ETangentOptions::BlendOverlappingNormals) != 0;
	bool bIgnoreDegenerateTriangles = (TangentOptions & ETangentOptions::IgnoreDegenerateTriangles) != 0;
	float ComparisonThreshold = bIgnoreDegenerateTriangles ? THRESH_POINTS_ARE_SAME : 0.0f;

	// Compute per-triangle tangents.
	TArray<FVector3f> TriangleTangentX;
	TArray<FVector3f> TriangleTangentY;
	TArray<FVector3f> TriangleTangentZ;

	ComputeTriangleTangents(
		InVertices,
		InIndices,
		InUVs,
		TriangleTangentX,
		TriangleTangentY,
		TriangleTangentZ,
		bIgnoreDegenerateTriangles ? SMALL_NUMBER : FLT_MIN
		);

	// Declare these out here to avoid reallocations.
	TArray<FFanFace> RelevantFacesForCorner[3];
	TArray<int32> AdjacentFaces;

	int32 NumWedges = InIndices.Num();
	int32 NumFaces = NumWedges / 3;

	// Allocate storage for tangents if none were provided.
	if (OutTangentX.Num() != NumWedges)
	{
		OutTangentX.Empty(NumWedges);
		OutTangentX.AddZeroed(NumWedges);
	}
	if (OutTangentY.Num() != NumWedges)
	{
		OutTangentY.Empty(NumWedges);
		OutTangentY.AddZeroed(NumWedges);
	}
	if (OutTangentZ.Num() != NumWedges)
	{
		OutTangentZ.Empty(NumWedges);
		OutTangentZ.AddZeroed(NumWedges);
	}

	for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
	{
		int32 WedgeOffset = FaceIndex * 3;
		FVector3f CornerPositions[3];
		FVector3f CornerTangentX[3];
		FVector3f CornerTangentY[3];
		FVector3f CornerTangentZ[3];

		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			CornerTangentX[CornerIndex] = FVector3f::ZeroVector;
			CornerTangentY[CornerIndex] = FVector3f::ZeroVector;
			CornerTangentZ[CornerIndex] = FVector3f::ZeroVector;
			CornerPositions[CornerIndex] = InVertices[InIndices[WedgeOffset + CornerIndex]];
			RelevantFacesForCorner[CornerIndex].Reset();
		}

		// Don't process degenerate triangles.
		if (PointsEqual(CornerPositions[0], CornerPositions[1], ComparisonThreshold)
			|| PointsEqual(CornerPositions[0], CornerPositions[2], ComparisonThreshold)
			|| PointsEqual(CornerPositions[1], CornerPositions[2], ComparisonThreshold))
		{
			continue;
		}

		// No need to process triangles if tangents already exist.
		bool bCornerHasTangents[3] = { 0 };
		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			bCornerHasTangents[CornerIndex] = !OutTangentX[WedgeOffset + CornerIndex].IsZero()
				&& !OutTangentY[WedgeOffset + CornerIndex].IsZero()
				&& !OutTangentZ[WedgeOffset + CornerIndex].IsZero();
		}
		if (bCornerHasTangents[0] && bCornerHasTangents[1] && bCornerHasTangents[2])
		{
			continue;
		}

		// Calculate smooth vertex normals.
		float Determinant = FVector3f::Triple(
			TriangleTangentX[FaceIndex],
			TriangleTangentY[FaceIndex],
			TriangleTangentZ[FaceIndex]
			);

		// Start building a list of faces adjacent to this face.
		AdjacentFaces.Reset();
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			int32 ThisCornerIndex = WedgeOffset + CornerIndex;
			const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(ThisCornerIndex);
			for (int32 k = 0, nk = DupVerts.Num() ; k < nk; k++)
			{
				AdjacentFaces.Add(DupVerts[k] / 3);
			}
			if (DupVerts.Num() == 0)
			{
				AdjacentFaces.Add(ThisCornerIndex / 3); // I am a "dup" of myself
			}
		}

		// We need to sort these here because the criteria for point equality is
		// exact, so we must ensure the exact same order for all dups.
 		AdjacentFaces.Sort();

		// Process adjacent faces
		int32 LastIndex = -1;
		for (int32 OtherFaceIndex : AdjacentFaces)
		{
			if (LastIndex == OtherFaceIndex)
			{
				continue;
			}

			LastIndex = OtherFaceIndex;

			for (int32 OurCornerIndex = 0; OurCornerIndex < 3; ++OurCornerIndex)
			{
				if (bCornerHasTangents[OurCornerIndex])
					continue;

				FFanFace NewFanFace;
				int32 CommonIndexCount = 0;

				// Check for vertices in common.
				if (FaceIndex == OtherFaceIndex)
				{
					CommonIndexCount = 3;
					NewFanFace.LinkedVertexIndex = OurCornerIndex;
				}
				else
				{
					// Check matching vertices against main vertex .
					for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; ++OtherCornerIndex)
					{
						if(CornerPositions[OurCornerIndex].Equals(InVertices[InIndices[OtherFaceIndex * 3 + OtherCornerIndex]], ComparisonThreshold))
						{
							CommonIndexCount++;
							NewFanFace.LinkedVertexIndex = OtherCornerIndex;
						}
					}
				}

				// Add if connected by at least one point. Smoothing matches are considered later.
				if (CommonIndexCount > 0)
				{
					NewFanFace.FaceIndex = OtherFaceIndex;
					NewFanFace.bFilled = (OtherFaceIndex == FaceIndex); // Starter face for smoothing floodfill.
					NewFanFace.bBlendTangents = NewFanFace.bFilled;
					NewFanFace.bBlendNormals = NewFanFace.bFilled;
					RelevantFacesForCorner[OurCornerIndex].Add(NewFanFace);
				}
			}
		}

		// Find true relevance of faces for a vertex normal by traversing
		// smoothing-group-compatible connected triangle fans around common vertices.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			if (bCornerHasTangents[CornerIndex])
				continue;

			int32 NewConnections;
			do
			{
				NewConnections = 0;
				for (int32 OtherFaceIdx = 0, ni = RelevantFacesForCorner[CornerIndex].Num() ; OtherFaceIdx < ni; ++OtherFaceIdx)
				{
					FFanFace& OtherFace = RelevantFacesForCorner[CornerIndex][OtherFaceIdx];
					// The vertex' own face is initially the only face with bFilled == true.
					if (OtherFace.bFilled)
					{
						for (int32 NextFaceIndex = 0, nk = RelevantFacesForCorner[CornerIndex].Num() ; NextFaceIndex < nk ; ++NextFaceIndex)
						{
							FFanFace& NextFace = RelevantFacesForCorner[CornerIndex][NextFaceIndex];
							if (!NextFace.bFilled) // && !NextFace.bBlendTangents)
							{
								if ((NextFaceIndex != OtherFaceIdx)
									&& (SmoothingGroupIndices[NextFace.FaceIndex] & SmoothingGroupIndices[OtherFace.FaceIndex]))
								{
									int32 CommonVertices = 0;
									int32 CommonTangentVertices = 0;
									int32 CommonNormalVertices = 0;
									for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; ++OtherCornerIndex)
									{
										for (int32 NextCornerIndex = 0; NextCornerIndex < 3; ++NextCornerIndex)
										{
											int32 NextVertexIndex = InIndices[NextFace.FaceIndex * 3 + NextCornerIndex];
											int32 OtherVertexIndex = InIndices[OtherFace.FaceIndex * 3 + OtherCornerIndex];
											if (PointsEqual(
												InVertices[NextVertexIndex],
												InVertices[OtherVertexIndex],
												ComparisonThreshold))
											{
												CommonVertices++;


												const FVector2f& UVOne = InUVs[NextFace.FaceIndex * 3 + NextCornerIndex];
												const FVector2f& UVTwo = InUVs[OtherFace.FaceIndex * 3 + OtherCornerIndex];

												if (UVsEqual(UVOne, UVTwo))
												{
													CommonTangentVertices++;
												}
												if (bBlendOverlappingNormals
													|| NextVertexIndex == OtherVertexIndex)
												{
													CommonNormalVertices++;
												}
											}
										}
									}
									// Flood fill faces with more than one common vertices which must be touching edges.
									if (CommonVertices > 1)
									{
										NextFace.bFilled = true;
										NextFace.bBlendNormals = (CommonNormalVertices > 1);
										NewConnections++;

										// Only blend tangents if there is no UV seam along the edge with this face.
										if (OtherFace.bBlendTangents && CommonTangentVertices > 1)
										{
											float OtherDeterminant = FVector3f::Triple(
												TriangleTangentX[NextFace.FaceIndex],
												TriangleTangentY[NextFace.FaceIndex],
												TriangleTangentZ[NextFace.FaceIndex]
												);
											if ((Determinant * OtherDeterminant) > 0.0f)
											{
												NextFace.bBlendTangents = true;
											}
										}
									}
								}
							}
						}
					}
				}
			} while (NewConnections > 0);
		}

		// Vertex normal construction.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			if (bCornerHasTangents[CornerIndex])
			{
				CornerTangentX[CornerIndex] = OutTangentX[WedgeOffset + CornerIndex];
				CornerTangentY[CornerIndex] = OutTangentY[WedgeOffset + CornerIndex];
				CornerTangentZ[CornerIndex] = OutTangentZ[WedgeOffset + CornerIndex];
			}
			else
			{
				for (int32 RelevantFaceIdx = 0; RelevantFaceIdx < RelevantFacesForCorner[CornerIndex].Num(); ++RelevantFaceIdx)
				{
					FFanFace const& RelevantFace = RelevantFacesForCorner[CornerIndex][RelevantFaceIdx];
					if (RelevantFace.bFilled)
					{
						int32 OtherFaceIndex = RelevantFace.FaceIndex;
						if (RelevantFace.bBlendTangents)
						{
							CornerTangentX[CornerIndex] += TriangleTangentX[OtherFaceIndex];
							CornerTangentY[CornerIndex] += TriangleTangentY[OtherFaceIndex];
						}
						if (RelevantFace.bBlendNormals)
						{
							CornerTangentZ[CornerIndex] += TriangleTangentZ[OtherFaceIndex];
						}
					}
				}
				if (!OutTangentX[WedgeOffset + CornerIndex].IsZero())
				{
					CornerTangentX[CornerIndex] = OutTangentX[WedgeOffset + CornerIndex];
				}
				if (!OutTangentY[WedgeOffset + CornerIndex].IsZero())
				{
					CornerTangentY[CornerIndex] = OutTangentY[WedgeOffset + CornerIndex];
				}
				if (!OutTangentZ[WedgeOffset + CornerIndex].IsZero())
				{
					CornerTangentZ[CornerIndex] = OutTangentZ[WedgeOffset + CornerIndex];
				}
			}
		}

		// Normalization.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			CornerTangentX[CornerIndex].Normalize();
			CornerTangentY[CornerIndex].Normalize();
			CornerTangentZ[CornerIndex].Normalize();

			// Gram-Schmidt orthogonalization
			CornerTangentY[CornerIndex] -= CornerTangentX[CornerIndex] * (CornerTangentX[CornerIndex] | CornerTangentY[CornerIndex]);
			CornerTangentY[CornerIndex].Normalize();

			CornerTangentX[CornerIndex] -= CornerTangentZ[CornerIndex] * (CornerTangentZ[CornerIndex] | CornerTangentX[CornerIndex]);
			CornerTangentX[CornerIndex].Normalize();
			CornerTangentY[CornerIndex] -= CornerTangentZ[CornerIndex] * (CornerTangentZ[CornerIndex] | CornerTangentY[CornerIndex]);
			CornerTangentY[CornerIndex].Normalize();
		}

		// Copy back to the mesh.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			OutTangentX[WedgeOffset + CornerIndex] = CornerTangentX[CornerIndex];
			OutTangentY[WedgeOffset + CornerIndex] = CornerTangentY[CornerIndex];
			OutTangentZ[WedgeOffset + CornerIndex] = CornerTangentZ[CornerIndex];
		}
	}

	check(OutTangentX.Num() == NumWedges);
	check(OutTangentY.Num() == NumWedges);
	check(OutTangentZ.Num() == NumWedges);
}


static void ComputeTangents(
	FRawMesh& RawMesh,
	const FOverlappingCorners& OverlappingCorners,
	uint32 TangentOptions
	)
{
	ComputeTangents(RawMesh.VertexPositions, RawMesh.WedgeIndices, RawMesh.WedgeTexCoords[0], RawMesh.FaceSmoothingMasks, OverlappingCorners, RawMesh.WedgeTangentX, RawMesh.WedgeTangentY, RawMesh.WedgeTangentZ, TangentOptions);
}

/*------------------------------------------------------------------------------
MikkTSpace for computing tangents.
------------------------------------------------------------------------------*/
class MikkTSpace_Mesh
{
public:
	const TArray<FVector3f>& Vertices;
	const TArray<uint32>& Indices;
	const TArray<FVector2f>& UVs;

	TArray<FVector3f>& TangentsX;			//Reference to newly created tangents list.
	TArray<FVector3f>& TangentsY;			//Reference to newly created bitangents list.
	const TArray<FVector3f>& TangentsZ;	//Reference to computed normals, will be empty otherwise.

	MikkTSpace_Mesh(
		const TArray<FVector3f>&		InVertices,
		const TArray<uint32>&		InIndices,
		const TArray<FVector2f>&	InUVs,
		TArray<FVector3f>&			InOutVertexTangentsX,
		TArray<FVector3f>&			InOutVertexTangentsY,
		const TArray<FVector3f>&		InVertexTangentsZ
		)
		:
		Vertices(InVertices),
		Indices(InIndices),
		UVs(InUVs),
		TangentsX(InOutVertexTangentsX),
		TangentsY(InOutVertexTangentsY),
		TangentsZ(InVertexTangentsZ)
	{
	}
};

static int MikkGetNumFaces(const SMikkTSpaceContext* Context)
{
	MikkTSpace_Mesh *UserData = (MikkTSpace_Mesh*)(Context->m_pUserData);
	return UserData->Indices.Num() / 3;
}

static int MikkGetNumVertsOfFace(const SMikkTSpaceContext* Context, const int FaceIdx)
{
	// All of our meshes are triangles.
	return 3;
}

static void MikkGetPosition(const SMikkTSpaceContext* Context, float Position[3], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Mesh *UserData = (MikkTSpace_Mesh*)(Context->m_pUserData);
	FVector3f VertexPosition = UserData->Vertices[ UserData->Indices[FaceIdx * 3 + VertIdx] ];
	Position[0] = VertexPosition.X;
	Position[1] = VertexPosition.Y;
	Position[2] = VertexPosition.Z;
}

static void MikkGetNormal(const SMikkTSpaceContext* Context, float Normal[3], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Mesh *UserData = (MikkTSpace_Mesh*)(Context->m_pUserData);
	const FVector3f& VertexNormal = UserData->TangentsZ[FaceIdx * 3 + VertIdx];
	for (int32 i = 0; i < 3; ++i)
	{
		Normal[i] = VertexNormal[i];
	}
}

static void MikkSetTSpaceBasic(const SMikkTSpaceContext* Context, const float Tangent[3], const float BitangentSign, const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Mesh *UserData = (MikkTSpace_Mesh*)(Context->m_pUserData);
	FVector3f &VertexTangent = UserData->TangentsX[FaceIdx * 3 + VertIdx];
	for (int32 i = 0; i < 3; ++i)
	{
		VertexTangent[i] = Tangent[i];
	}
	FVector3f Bitangent = BitangentSign * FVector3f::CrossProduct(UserData->TangentsZ[FaceIdx * 3 + VertIdx], VertexTangent);
	FVector3f &VertexBitangent = UserData->TangentsY[FaceIdx * 3 + VertIdx];
	for (int32 i = 0; i < 3; ++i)
	{
		VertexBitangent[i] = -Bitangent[i];
	}
}

static void MikkGetTexCoord(const SMikkTSpaceContext* Context, float UV[2], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Mesh *UserData = (MikkTSpace_Mesh*)(Context->m_pUserData);
	const FVector2f &TexCoord = UserData->UVs[FaceIdx * 3 + VertIdx];
	UV[0] = TexCoord.X;
	UV[1] = TexCoord.Y;
}

// MikkTSpace implementations for skeletal meshes, where tangents/bitangents are ultimately derived from lists of attributes.

// Holder for skeletal data to be passed to MikkTSpace.
// Holds references to the wedge, face and points vectors that BuildSkeletalMesh is given.
// Holds reference to the calculated normals array, which will be fleshed out if they've been calculated.
// Holds reference to the newly created tangent and bitangent arrays, which MikkTSpace will fleshed out if required.
class MikkTSpace_Skeletal_Mesh
{
public:
	const TArray<SkeletalMeshImportData::FMeshWedge>	&wedges;			//Reference to wedge list.
	const TArray<SkeletalMeshImportData::FMeshFace>		&faces;				//Reference to face list.	Also contains normal/tangent/bitanget/UV coords for each vertex of the face.
	const TArray<FVector3f>		&points;			//Reference to position list.
	bool						bComputeNormals;	//Copy of bComputeNormals.
	TArray<FVector3f>				&TangentsX;			//Reference to newly created tangents list.
	TArray<FVector3f>				&TangentsY;			//Reference to newly created bitangents list.
	TArray<FVector3f>				&TangentsZ;			//Reference to computed normals, will be empty otherwise.

	MikkTSpace_Skeletal_Mesh(
		const TArray<SkeletalMeshImportData::FMeshWedge>	&Wedges,
		const TArray<SkeletalMeshImportData::FMeshFace>		&Faces,
		const TArray<FVector3f>		&Points,
		bool						bInComputeNormals,
		TArray<FVector3f>				&VertexTangentsX,
		TArray<FVector3f>				&VertexTangentsY,
		TArray<FVector3f>				&VertexTangentsZ
		)
		:
		wedges(Wedges),
		faces(Faces),
		points(Points),
		bComputeNormals(bInComputeNormals),
		TangentsX(VertexTangentsX),
		TangentsY(VertexTangentsY),
		TangentsZ(VertexTangentsZ)
	{
	}
};

static int MikkGetNumFaces_Skeletal(const SMikkTSpaceContext* Context)
{
	MikkTSpace_Skeletal_Mesh *UserData = (MikkTSpace_Skeletal_Mesh*)(Context->m_pUserData);
	return UserData->faces.Num();
}

static int MikkGetNumVertsOfFace_Skeletal(const SMikkTSpaceContext* Context, const int FaceIdx)
{
	// Confirmed?
	return 3;
}

static void MikkGetPosition_Skeletal(const SMikkTSpaceContext* Context, float Position[3], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Skeletal_Mesh *UserData = (MikkTSpace_Skeletal_Mesh*)(Context->m_pUserData);
	const FVector3f &VertexPosition = UserData->points[UserData->wedges[UserData->faces[FaceIdx].iWedge[VertIdx]].iVertex];
	Position[0] = VertexPosition.X;
	Position[1] = VertexPosition.Y;
	Position[2] = VertexPosition.Z;
}

static void MikkGetNormal_Skeletal(const SMikkTSpaceContext* Context, float Normal[3], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Skeletal_Mesh *UserData = (MikkTSpace_Skeletal_Mesh*)(Context->m_pUserData);
	// Get different normals depending on whether they've been calculated or not.
	if (UserData->bComputeNormals) {
		FVector3f &VertexNormal = UserData->TangentsZ[FaceIdx * 3 + VertIdx];
		Normal[0] = VertexNormal.X;
		Normal[1] = VertexNormal.Y;
		Normal[2] = VertexNormal.Z;
	}
	else
	{
		const FVector3f &VertexNormal = UserData->faces[FaceIdx].TangentZ[VertIdx];
		Normal[0] = VertexNormal.X;
		Normal[1] = VertexNormal.Y;
		Normal[2] = VertexNormal.Z;
	}
}

static void MikkSetTSpaceBasic_Skeletal(const SMikkTSpaceContext* Context, const float Tangent[3], const float BitangentSign, const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Skeletal_Mesh *UserData = (MikkTSpace_Skeletal_Mesh*)(Context->m_pUserData);
	FVector3f &VertexTangent = UserData->TangentsX[FaceIdx * 3 + VertIdx];
	VertexTangent.X = Tangent[0];
	VertexTangent.Y = Tangent[1];
	VertexTangent.Z = Tangent[2];

	FVector3f Bitangent;
	// Get different normals depending on whether they've been calculated or not.
	if (UserData->bComputeNormals) {
		Bitangent = BitangentSign * FVector3f::CrossProduct(UserData->TangentsZ[FaceIdx * 3 + VertIdx], VertexTangent);
	}
	else
	{
		Bitangent = BitangentSign * FVector3f::CrossProduct(UserData->faces[FaceIdx].TangentZ[VertIdx], VertexTangent);
	}
	FVector3f &VertexBitangent = UserData->TangentsY[FaceIdx * 3 + VertIdx];
	// Switch the tangent space swizzle to X+Y-Z+ for legacy reasons.
	VertexBitangent.X = -Bitangent[0];
	VertexBitangent.Y = -Bitangent[1];
	VertexBitangent.Z = -Bitangent[2];
}

static void MikkGetTexCoord_Skeletal(const SMikkTSpaceContext* Context, float UV[2], const int FaceIdx, const int VertIdx)
{
	MikkTSpace_Skeletal_Mesh *UserData = (MikkTSpace_Skeletal_Mesh*)(Context->m_pUserData);
	const FVector2f &TexCoord = UserData->wedges[UserData->faces[FaceIdx].iWedge[VertIdx]].UVs[0];
	UV[0] = TexCoord.X;
	UV[1] = TexCoord.Y;
}

static void ComputeNormals(
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	const TArray<FVector2f>& InUVs,
	const TArray<uint32>& SmoothingGroupIndices,
	const FOverlappingCorners& OverlappingCorners,
	TArray<FVector3f>& OutTangentZ,
	const uint32 TangentOptions
	)
{
	bool bBlendOverlappingNormals = (TangentOptions & ETangentOptions::BlendOverlappingNormals) != 0;
	bool bIgnoreDegenerateTriangles = (TangentOptions & ETangentOptions::IgnoreDegenerateTriangles) != 0;
	float ComparisonThreshold = bIgnoreDegenerateTriangles ? THRESH_POINTS_ARE_SAME : 0.0f;

	// Compute per-triangle tangents.
	TArray<FVector3f> TriangleTangentX;
	TArray<FVector3f> TriangleTangentY;
	TArray<FVector3f> TriangleTangentZ;

	ComputeTriangleTangents(
		InVertices,
		InIndices,
		InUVs,
		TriangleTangentX,
		TriangleTangentY,
		TriangleTangentZ,
		bIgnoreDegenerateTriangles ? SMALL_NUMBER : FLT_MIN
		);

	// Declare these out here to avoid reallocations.
	TArray<FFanFace> RelevantFacesForCorner[3];
// 	TArray<int32> AdjacentFaces;

	int32 NumWedges = InIndices.Num();
	int32 NumFaces = NumWedges / 3;

	// Allocate storage for tangents if none were provided, and calculate normals for MikkTSpace.
	if (OutTangentZ.Num() != NumWedges)
	{
		// normals are not included, so we should calculate them
		OutTangentZ.Empty(NumWedges);
		OutTangentZ.AddZeroed(NumWedges);
	}

	// we need to calculate normals for MikkTSpace
	for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
	{
		int32 WedgeOffset = FaceIndex * 3;
		FVector3f CornerPositions[3];
		FVector3f CornerNormal[3];

		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			CornerNormal[CornerIndex] = FVector3f::ZeroVector;
			CornerPositions[CornerIndex] = InVertices[InIndices[WedgeOffset + CornerIndex]];
			RelevantFacesForCorner[CornerIndex].Reset();
		}

		// Don't process degenerate triangles.
		if (PointsEqual(CornerPositions[0], CornerPositions[1], ComparisonThreshold)
			|| PointsEqual(CornerPositions[0], CornerPositions[2], ComparisonThreshold)
			|| PointsEqual(CornerPositions[1], CornerPositions[2], ComparisonThreshold))
		{
			continue;
		}

		// No need to process triangles if tangents already exist.
		bool bCornerHasNormal[3] = { 0 };
		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			bCornerHasNormal[CornerIndex] = !OutTangentZ[WedgeOffset + CornerIndex].IsZero();
		}
		if (bCornerHasNormal[0] && bCornerHasNormal[1] && bCornerHasNormal[2])
		{
			continue;
		}

		// Start building a list of faces adjacent to this face.
// 		AdjacentFaces.Reset();
		TSet<int32> AdjacentFaces;
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			int32 ThisCornerIndex = WedgeOffset + CornerIndex;
			const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(ThisCornerIndex);
			if (DupVerts.Num() == 0)
			{
//				AdjacentFaces.AddUnique(ThisCornerIndex / 3); // I am a "dup" of myself
				AdjacentFaces.Add(ThisCornerIndex / 3); // I am a "dup" of myself
			}
			for (int32 k = 0; k < DupVerts.Num(); k++)
			{
				AdjacentFaces.Add(DupVerts[k] / 3);
			}
		}

		// We need to sort these here because the criteria for point equality is
		// exact, so we must ensure the exact same order for all dups.
// 		AdjacentFaces.Sort();

		// Process adjacent faces
// 		for (int32 AdjacentFaceIndex = 0; AdjacentFaceIndex < AdjacentFaces.Num(); AdjacentFaceIndex++)
		for (int32 OtherFaceIndex : AdjacentFaces )
		{
// 			int32 OtherFaceIndex = AdjacentFaces[AdjacentFaceIndex];
			for (int32 OurCornerIndex = 0; OurCornerIndex < 3; ++OurCornerIndex)
			{
				if (bCornerHasNormal[OurCornerIndex])
					continue;

				FFanFace NewFanFace;
				int32 CommonIndexCount = 0;

				// Check for vertices in common.
				if (FaceIndex == OtherFaceIndex)
				{
					CommonIndexCount = 3;
					NewFanFace.LinkedVertexIndex = OurCornerIndex;
				}
				else
				{
					// Check matching vertices against main vertex .
					for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; OtherCornerIndex++)
					{
						if (PointsEqual(
							CornerPositions[OurCornerIndex],
							InVertices[InIndices[OtherFaceIndex * 3 + OtherCornerIndex]],
							ComparisonThreshold
							))
						{
							CommonIndexCount++;
							NewFanFace.LinkedVertexIndex = OtherCornerIndex;
						}
					}
				}

				// Add if connected by at least one point. Smoothing matches are considered later.
				if (CommonIndexCount > 0)
				{
					NewFanFace.FaceIndex = OtherFaceIndex;
					NewFanFace.bFilled = (OtherFaceIndex == FaceIndex); // Starter face for smoothing floodfill.
					NewFanFace.bBlendTangents = NewFanFace.bFilled;
					NewFanFace.bBlendNormals = NewFanFace.bFilled;
					RelevantFacesForCorner[OurCornerIndex].Add(NewFanFace);
				}
			}
		}

		// Find true relevance of faces for a vertex normal by traversing
		// smoothing-group-compatible connected triangle fans around common vertices.
		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			if (bCornerHasNormal[CornerIndex])
				continue;

			int32 NewConnections;
			do
			{
				NewConnections = 0;
				for (int32 OtherFaceIdx = 0; OtherFaceIdx < RelevantFacesForCorner[CornerIndex].Num(); OtherFaceIdx++)
				{
					FFanFace& OtherFace = RelevantFacesForCorner[CornerIndex][OtherFaceIdx];
					// The vertex' own face is initially the only face with bFilled == true.
					if (OtherFace.bFilled)
					{
						for (int32 NextFaceIndex = 0; NextFaceIndex < RelevantFacesForCorner[CornerIndex].Num(); NextFaceIndex++)
						{
							FFanFace& NextFace = RelevantFacesForCorner[CornerIndex][NextFaceIndex];
							if (!NextFace.bFilled) // && !NextFace.bBlendTangents)
							{
								if ((NextFaceIndex != OtherFaceIdx)
									&& (SmoothingGroupIndices[NextFace.FaceIndex] & SmoothingGroupIndices[OtherFace.FaceIndex]))
								{
									int32 CommonVertices = 0;
									int32 CommonNormalVertices = 0;
									for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; ++OtherCornerIndex)
									{
										for (int32 NextCornerIndex = 0; NextCornerIndex < 3; ++NextCornerIndex)
										{
											int32 NextVertexIndex = InIndices[NextFace.FaceIndex * 3 + NextCornerIndex];
											int32 OtherVertexIndex = InIndices[OtherFace.FaceIndex * 3 + OtherCornerIndex];
											if (PointsEqual(
												InVertices[NextVertexIndex],
												InVertices[OtherVertexIndex],
												ComparisonThreshold))
											{
												CommonVertices++;
												if (bBlendOverlappingNormals
													|| NextVertexIndex == OtherVertexIndex)
												{
													CommonNormalVertices++;
												}
											}
										}
									}
									// Flood fill faces with more than one common vertices which must be touching edges.
									if (CommonVertices > 1)
									{
										NextFace.bFilled = true;
										NextFace.bBlendNormals = (CommonNormalVertices > 1);
										NewConnections++;
									}
								}
							}
						}
					}
				}
			} 
			while (NewConnections > 0);
		}


		// Vertex normal construction.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			if (bCornerHasNormal[CornerIndex])
			{
				CornerNormal[CornerIndex] = OutTangentZ[WedgeOffset + CornerIndex];
			}
			else
			{
				for (int32 RelevantFaceIdx = 0; RelevantFaceIdx < RelevantFacesForCorner[CornerIndex].Num(); RelevantFaceIdx++)
				{
					FFanFace const& RelevantFace = RelevantFacesForCorner[CornerIndex][RelevantFaceIdx];
					if (RelevantFace.bFilled)
					{
						int32 OtherFaceIndex = RelevantFace.FaceIndex;
						if (RelevantFace.bBlendNormals)
						{
							CornerNormal[CornerIndex] += TriangleTangentZ[OtherFaceIndex];
						}
					}
				}
				if (!OutTangentZ[WedgeOffset + CornerIndex].IsZero())
				{
					CornerNormal[CornerIndex] = OutTangentZ[WedgeOffset + CornerIndex];
				}
			}
		}

		// Normalization.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			CornerNormal[CornerIndex].Normalize();
		}

		// Copy back to the mesh.
		for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
		{
			OutTangentZ[WedgeOffset + CornerIndex] = CornerNormal[CornerIndex];
		}
	}

	check(OutTangentZ.Num() == NumWedges);
}

static void ComputeTangents_MikkTSpace(
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	const TArray<FVector2f>& InUVs,
	const TArray<FVector3f>& InNormals,
	bool bIgnoreDegenerateTriangles,
	TArray<FVector3f>& OutTangentX,
	TArray<FVector3f>& OutTangentY
	)
{
	const int32 NumWedges = InIndices.Num();

	bool bTangentsComputationNeeded = false;

	if (OutTangentX.Num() != NumWedges)
	{
		OutTangentX.Empty(NumWedges);
		OutTangentX.AddZeroed(NumWedges);
		bTangentsComputationNeeded = true;
	}
	if (OutTangentY.Num() != NumWedges)
	{
		OutTangentY.Empty(NumWedges);
		OutTangentY.AddZeroed(NumWedges);
		bTangentsComputationNeeded = true;
	}

	if (!bTangentsComputationNeeded && NumWedges > 0)
	{
		for (int32 WedgeIdx = 0; WedgeIdx < NumWedges && !bTangentsComputationNeeded; ++WedgeIdx)
		{
			bTangentsComputationNeeded = OutTangentX[WedgeIdx].IsNearlyZero() || OutTangentY[WedgeIdx].IsNearlyZero();
		}
	}

	if (!bTangentsComputationNeeded)
	{
		return;
	}

	MikkTSpace_Mesh MikkTSpaceMesh(InVertices, InIndices, InUVs, OutTangentX, OutTangentY, InNormals);

	// use mikktspace to calculate the tangents
	SMikkTSpaceInterface MikkTInterface;
	MikkTInterface.m_getNormal = MikkGetNormal;
	MikkTInterface.m_getNumFaces = MikkGetNumFaces;
	MikkTInterface.m_getNumVerticesOfFace = MikkGetNumVertsOfFace;
	MikkTInterface.m_getPosition = MikkGetPosition;
	MikkTInterface.m_getTexCoord = MikkGetTexCoord;
	MikkTInterface.m_setTSpaceBasic = MikkSetTSpaceBasic;
	MikkTInterface.m_setTSpace = nullptr;

	SMikkTSpaceContext MikkTContext;
	MikkTContext.m_pInterface = &MikkTInterface;
	MikkTContext.m_pUserData = (void*)(&MikkTSpaceMesh);
	MikkTContext.m_bIgnoreDegenerates = bIgnoreDegenerateTriangles;
	genTangSpaceDefault(&MikkTContext);
}

static void ComputeTangents_MikkTSpace(
	const TArray<FVector3f>& InVertices,
	const TArray<uint32>& InIndices,
	const TArray<FVector2f>& InUVs,
	const TArray<uint32>& SmoothingGroupIndices,
	const FOverlappingCorners& OverlappingCorners,
	TArray<FVector3f>& OutTangentX,
	TArray<FVector3f>& OutTangentY,
	TArray<FVector3f>& OutTangentZ,
	const uint32 TangentOptions
	)
{
	ComputeNormals( InVertices, InIndices, InUVs, SmoothingGroupIndices, OverlappingCorners, OutTangentZ, TangentOptions );

	bool bIgnoreDegenerateTriangles = (TangentOptions & ETangentOptions::IgnoreDegenerateTriangles) != 0;

	int32 NumWedges = InIndices.Num();

	ComputeTangents_MikkTSpace(InVertices, InIndices, InUVs, OutTangentZ, bIgnoreDegenerateTriangles, OutTangentX, OutTangentY);

	check(OutTangentX.Num() == NumWedges);
	check(OutTangentY.Num() == NumWedges);
	check(OutTangentZ.Num() == NumWedges);
}

static void ComputeTangents_MikkTSpace(
	FRawMesh& RawMesh,
	const FOverlappingCorners& OverlappingCorners,
	uint32 TangentOptions
	)
{
	ComputeTangents_MikkTSpace(RawMesh.VertexPositions, RawMesh.WedgeIndices, RawMesh.WedgeTexCoords[0], RawMesh.FaceSmoothingMasks, OverlappingCorners, RawMesh.WedgeTangentX, RawMesh.WedgeTangentY, RawMesh.WedgeTangentZ, TangentOptions);
}

static void BuildDepthOnlyIndexBuffer(
	TArray<uint32>& OutDepthIndices,
	const TArray<FStaticMeshBuildVertex>& InVertices,
	const TArray<uint32>& InIndices,
	const TArrayView<FStaticMeshSection>& InSections
	)
{
	int32 NumVertices = InVertices.Num();
	if (InIndices.Num() <= 0 || NumVertices <= 0)
	{
		OutDepthIndices.Empty();
		return;
	}

	// Create a mapping of index -> first overlapping index to accelerate the construction of the shadow index buffer.
	TArray<FIndexAndZ> VertIndexAndZ;
	VertIndexAndZ.Empty(NumVertices);
	for (int32 VertIndex = 0; VertIndex < NumVertices; VertIndex++)
	{
		new(VertIndexAndZ)FIndexAndZ(VertIndex, InVertices[VertIndex].Position);
	}
	VertIndexAndZ.Sort(FCompareIndexAndZ());

	// Setup the index map. 0xFFFFFFFF == not set.
	TArray<uint32> IndexMap;
	IndexMap.AddUninitialized(NumVertices);
	FMemory::Memset(IndexMap.GetData(), 0xFF, NumVertices * sizeof(uint32));

	// Search for duplicates, quickly!
	for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
	{
		uint32 SrcIndex = VertIndexAndZ[i].Index;
		float Z = VertIndexAndZ[i].Z;
		IndexMap[SrcIndex] = FMath::Min(IndexMap[SrcIndex], SrcIndex);

		// Search forward since we add pairs both ways.
		for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
		{
			if (FMath::Abs(VertIndexAndZ[j].Z - Z) > THRESH_POINTS_ARE_SAME * 4.01f)
				break; // can't be any more dups

			uint32 OtherIndex = VertIndexAndZ[j].Index;
			if (PointsEqual(InVertices[SrcIndex].Position, InVertices[OtherIndex].Position,/*bUseEpsilonCompare=*/ false))
			{
				IndexMap[SrcIndex] = FMath::Min(IndexMap[SrcIndex], OtherIndex);
				IndexMap[OtherIndex] = FMath::Min(IndexMap[OtherIndex], SrcIndex);
			}
		}
	}

	// Build the depth-only index buffer by remapping all indices to the first overlapping
	// vertex in the vertex buffer.
	OutDepthIndices.Empty();
	for (int32 SectionIndex = 0; SectionIndex < InSections.Num(); ++SectionIndex)
	{
		const FStaticMeshSection& Section = InSections[SectionIndex];
		int32 FirstIndex = Section.FirstIndex;
		int32 LastIndex = FirstIndex + Section.NumTriangles * 3;
		for (int32 SrcIndex = FirstIndex; SrcIndex < LastIndex; ++SrcIndex)
		{
			uint32 VertIndex = InIndices[SrcIndex];
			OutDepthIndices.Add(IndexMap[VertIndex]);
		}
	}
}

static float GetComparisonThreshold(FMeshBuildSettings const& BuildSettings)
{
	return BuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
}

/*------------------------------------------------------------------------------
Static mesh building.
------------------------------------------------------------------------------*/

static void BuildStaticMeshVertex(const FRawMesh& RawMesh, const FMatrix44f& ScaleMatrix, const FVector3f& Position, int32 WedgeIndex, FStaticMeshBuildVertex& Vertex)
{
	Vertex.Position = Position;

	Vertex.TangentX = ScaleMatrix.TransformVector(RawMesh.WedgeTangentX[WedgeIndex]).GetSafeNormal();
	Vertex.TangentY = ScaleMatrix.TransformVector(RawMesh.WedgeTangentY[WedgeIndex]).GetSafeNormal();
	Vertex.TangentZ = ScaleMatrix.TransformVector(RawMesh.WedgeTangentZ[WedgeIndex]).GetSafeNormal();

	if (RawMesh.WedgeColors.IsValidIndex(WedgeIndex))
	{
		Vertex.Color = RawMesh.WedgeColors[WedgeIndex];
	}
	else
	{
		Vertex.Color = FColor::White;
	}

	static const int32 NumTexCoords = FMath::Min<int32>(MAX_MESH_TEXTURE_COORDS, MAX_STATIC_TEXCOORDS);
	for (int32 i = 0; i < NumTexCoords; ++i)
	{
		if (RawMesh.WedgeTexCoords[i].IsValidIndex(WedgeIndex))
		{
			Vertex.UVs[i] = RawMesh.WedgeTexCoords[i][WedgeIndex];
		}
		else
		{
			Vertex.UVs[i] = FVector2f(0.0f, 0.0f);
		}
	}
}

static bool AreVerticesEqual(
	FStaticMeshBuildVertex const& A,
	FStaticMeshBuildVertex const& B,
	float ComparisonThreshold
	)
{
	if (!PointsEqual(A.Position, B.Position, ComparisonThreshold)
		|| !NormalsEqual(A.TangentX, B.TangentX)
		|| !NormalsEqual(A.TangentY, B.TangentY)
		|| !NormalsEqual(A.TangentZ, B.TangentZ)
		|| A.Color != B.Color)
	{
		return false;
	}

	// UVs
	for (int32 UVIndex = 0; UVIndex < MAX_STATIC_TEXCOORDS; UVIndex++)
	{
		if (!UVsEqual(A.UVs[UVIndex], B.UVs[UVIndex]))
		{
			return false;
		}
	}

	return true;
}

void FMeshUtilities::BuildStaticMeshVertexAndIndexBuffers(
	TArray<FStaticMeshBuildVertex>& OutVertices,
	TArray<TArray<uint32> >& OutPerSectionIndices,
	TArray<int32>& OutWedgeMap,
	const FRawMesh& RawMesh,
	const FOverlappingCorners& OverlappingCorners,
	const TMap<uint32, uint32>& MaterialToSectionMapping,
	float ComparisonThreshold,
	FVector3f BuildScale,
	int32 ImportVersion
	)
{
	TMap<int32, int32> FinalVerts;
	int32 NumFaces = RawMesh.WedgeIndices.Num() / 3;
	OutWedgeMap.Reset(RawMesh.WedgeIndices.Num());
	FMatrix44f ScaleMatrix(FScaleMatrix44f(BuildScale).Inverse().GetTransposed());

	// Estimate how many vertices there will be to reduce number of re-allocations required
	OutVertices.Reserve((int32)(NumFaces * 1.2) + 16);

	// Work with vertex in OutVertices array directly for improved performance
	OutVertices.AddUninitialized(1);
	FStaticMeshBuildVertex *ThisVertex = &OutVertices.Last();

	// Process each face, build vertex buffer and per-section index buffers.
	for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
	{
		int32 VertexIndices[3];
		FVector3f CornerPositions[3];

		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			CornerPositions[CornerIndex] = GetPositionForWedge(RawMesh, FaceIndex * 3 + CornerIndex);
		}

		// Don't process degenerate triangles.
		if (PointsEqual(CornerPositions[0], CornerPositions[1], ComparisonThreshold)
			|| PointsEqual(CornerPositions[0], CornerPositions[2], ComparisonThreshold)
			|| PointsEqual(CornerPositions[1], CornerPositions[2], ComparisonThreshold))
		{
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				OutWedgeMap.Add(INDEX_NONE);
			}
			continue;
		}

		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			int32 WedgeIndex = FaceIndex * 3 + CornerIndex;
			BuildStaticMeshVertex(RawMesh, ScaleMatrix, CornerPositions[CornerIndex] * BuildScale, WedgeIndex, *ThisVertex);

			const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(WedgeIndex);

			int32 Index = INDEX_NONE;
			for (int32 k = 0; k < DupVerts.Num(); k++)
			{
				if (DupVerts[k] >= WedgeIndex)
				{
					// the verts beyond me haven't been placed yet, so these duplicates are not relevant
					break;
				}

				int32 *Location = FinalVerts.Find(DupVerts[k]);
				if (Location != NULL
					&& AreVerticesEqual(*ThisVertex, OutVertices[*Location], ComparisonThreshold))
				{
					Index = *Location;
					break;
				}
			}
			if (Index == INDEX_NONE)
			{
				// Commit working vertex
				Index = OutVertices.Num() - 1;
				FinalVerts.Add(WedgeIndex, Index);

				// Setup next working vertex
				OutVertices.AddUninitialized(1);
				ThisVertex = &OutVertices.Last();
			}
			VertexIndices[CornerIndex] = Index;
		}

		// Reject degenerate triangles.
		if (VertexIndices[0] == VertexIndices[1]
			|| VertexIndices[1] == VertexIndices[2]
			|| VertexIndices[0] == VertexIndices[2])
		{
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				OutWedgeMap.Add(INDEX_NONE);
			}
			continue;
		}

		// Put the indices in the material index buffer.
		uint32 SectionIndex = 0;
		if (ImportVersion < RemoveStaticMeshSkinxxWorkflow)
		{
			SectionIndex = FMath::Clamp(RawMesh.FaceMaterialIndices[FaceIndex], 0, OutPerSectionIndices.Num() - 1);
		}
		else
		{
			SectionIndex = MaterialToSectionMapping.FindChecked(RawMesh.FaceMaterialIndices[FaceIndex]);
		}
		TArray<uint32>& SectionIndices = OutPerSectionIndices[SectionIndex];
		for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
		{
			SectionIndices.Add(VertexIndices[CornerIndex]);
			OutWedgeMap.Add(VertexIndices[CornerIndex]);
		}
	}

	// Remove working vertex
	OutVertices.Pop(EAllowShrinking::No);
}

void FMeshUtilities::CacheOptimizeVertexAndIndexBuffer(
	TArray<FStaticMeshBuildVertex>& Vertices,
	TArray<TArray<uint32> >& PerSectionIndices,
	TArray<int32>& WedgeMap
	)
{
	// Copy the vertices since we will be reordering them
	TArray<FStaticMeshBuildVertex> OriginalVertices = Vertices;

	// Initialize a cache that stores which indices have been assigned
	TArray<int32> IndexCache;
	IndexCache.AddUninitialized(Vertices.Num());
	FMemory::Memset(IndexCache.GetData(), INDEX_NONE, IndexCache.Num() * IndexCache.GetTypeSize());
	int32 NextAvailableIndex = 0;

	// Iterate through the section index buffers, 
	// Optimizing index order for the post transform cache (minimizes the number of vertices transformed), 
	// And vertex order for the pre transform cache (minimizes the amount of vertex data fetched by the GPU).
	for (int32 SectionIndex = 0; SectionIndex < PerSectionIndices.Num(); SectionIndex++)
	{
		TArray<uint32>& Indices = PerSectionIndices[SectionIndex];

		if (Indices.Num())
		{
			// Optimize the index buffer for the post transform cache with.
			CacheOptimizeIndexBuffer(Indices);

			// Copy the index buffer since we will be reordering it
			TArray<uint32> OriginalIndices = Indices;

			// Go through the indices and assign them new values that are coherent where possible
			for (int32 Index = 0; Index < Indices.Num(); Index++)
			{
				const int32 CachedIndex = IndexCache[OriginalIndices[Index]];

				if (CachedIndex == INDEX_NONE)
				{
					// No new index has been allocated for this existing index, assign a new one
					Indices[Index] = NextAvailableIndex;
					// Mark what this index has been assigned to
					IndexCache[OriginalIndices[Index]] = NextAvailableIndex;
					NextAvailableIndex++;
				}
				else
				{
					// Reuse an existing index assignment
					Indices[Index] = CachedIndex;
				}
				// Reorder the vertices based on the new index assignment
				Vertices[Indices[Index]] = OriginalVertices[OriginalIndices[Index]];
			}
		}
	}

	for (int32 i = 0; i < WedgeMap.Num(); i++)
	{
		int32 MappedIndex = WedgeMap[i];
		if (MappedIndex != INDEX_NONE)
		{
			WedgeMap[i] = IndexCache[MappedIndex];
		}
	}
}

struct FLayoutUVRawMeshView final : FLayoutUV::IMeshView
{
	FRawMesh& RawMesh;
	const uint32 SrcChannel;
	const uint32 DstChannel;
	const bool bNormalsValid;

	FLayoutUVRawMeshView(FRawMesh& InRawMesh, uint32 InSrcChannel, uint32 InDstChannel) 
		: RawMesh(InRawMesh)
		, SrcChannel(InSrcChannel)
		, DstChannel(InDstChannel)
		, bNormalsValid(InRawMesh.WedgeTangentZ.Num() == InRawMesh.WedgeTexCoords[InSrcChannel].Num())
	{}

	uint32     GetNumIndices() const override { return RawMesh.WedgeIndices.Num(); }
	FVector3f  GetPosition(uint32 Index) const override { return RawMesh.GetWedgePosition(Index); }
	FVector3f  GetNormal(uint32 Index) const override { return bNormalsValid ? RawMesh.WedgeTangentZ[Index] : FVector3f::ZeroVector; }
	FVector2f  GetInputTexcoord(uint32 Index) const override { return RawMesh.WedgeTexCoords[SrcChannel][Index]; }

	void      InitOutputTexcoords(uint32 Num) override { RawMesh.WedgeTexCoords[DstChannel].SetNumUninitialized( Num ); }
	void      SetOutputTexcoord(uint32 Index, const FVector2f& Value) override { RawMesh.WedgeTexCoords[DstChannel][Index] = Value; }
};

class FStaticMeshUtilityBuilder
{
public:
	FStaticMeshUtilityBuilder(UStaticMesh* InStaticMesh) : Stage(EStage::Uninit), NumValidLODs(0), StaticMesh(InStaticMesh) {}

	bool GatherSourceMeshesPerLOD(IMeshReduction* MeshReduction)
	{
		check(Stage == EStage::Uninit);
		check(StaticMesh != nullptr);
		ELightmapUVVersion LightmapUVVersion = (ELightmapUVVersion)StaticMesh->GetLightmapUVVersion();

		FMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<FMeshUtilities>("MeshUtilities");

		// Gather source meshes for each LOD.
		int32 NumSourceModels = StaticMesh->GetNumSourceModels();
		for (int32 LODIndex = 0; LODIndex < NumSourceModels; ++LODIndex)
		{
			FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
			FRawMesh& RawMesh = *new FRawMesh;
			LODMeshes.Add(&RawMesh);
			FOverlappingCorners& OverlappingCorners = *new FOverlappingCorners;
			LODOverlappingCorners.Add(&OverlappingCorners);

			if (!SrcModel.IsRawMeshEmpty())
			{
				SrcModel.LoadRawMesh(RawMesh);
				// Make sure the raw mesh is not irreparably malformed.
				if (!RawMesh.IsValidOrFixable())
				{
					UE_LOGF(LogMeshUtilities, Error, "Raw mesh is corrupt for LOD%d.", LODIndex);
					return false;
				}
				LODBuildSettings[LODIndex] = SrcModel.BuildSettings;

				float ComparisonThreshold = GetComparisonThreshold(LODBuildSettings[LODIndex]);
				int32 NumWedges = RawMesh.WedgeIndices.Num();

				// Find overlapping corners to accelerate adjacency.
				MeshUtilities.FindOverlappingCorners(OverlappingCorners, RawMesh, ComparisonThreshold);

				// Figure out if we should recompute normals and tangents.
				bool bRecomputeNormals = SrcModel.BuildSettings.bRecomputeNormals || RawMesh.WedgeTangentZ.Num() != NumWedges;
				bool bRecomputeTangents = SrcModel.BuildSettings.bRecomputeTangents || RawMesh.WedgeTangentX.Num() != NumWedges || RawMesh.WedgeTangentY.Num() != NumWedges;

				// Dump normals and tangents if we are recomputing them.
				if (bRecomputeTangents)
				{
					RawMesh.WedgeTangentX.Empty(NumWedges);
					RawMesh.WedgeTangentX.AddZeroed(NumWedges);
					RawMesh.WedgeTangentY.Empty(NumWedges);
					RawMesh.WedgeTangentY.AddZeroed(NumWedges);
				}
				if (bRecomputeNormals)
				{
					RawMesh.WedgeTangentZ.Empty(NumWedges);
					RawMesh.WedgeTangentZ.AddZeroed(NumWedges);
				}

				// Compute any missing tangents.
				{
					// Static meshes always blend normals of overlapping corners.
					uint32 TangentOptions = ETangentOptions::BlendOverlappingNormals;
					if (SrcModel.BuildSettings.bRemoveDegenerates)
					{
						// If removing degenerate triangles, ignore them when computing tangents.
						TangentOptions |= ETangentOptions::IgnoreDegenerateTriangles;
					}

					//MikkTSpace should be use only when the user want to recompute the normals or tangents otherwise should always fallback on builtin
					if (SrcModel.BuildSettings.bUseMikkTSpace && (SrcModel.BuildSettings.bRecomputeNormals || SrcModel.BuildSettings.bRecomputeTangents))
					{
						ComputeTangents_MikkTSpace(RawMesh, OverlappingCorners, TangentOptions);
					}
					else
					{
						ComputeTangents(RawMesh, OverlappingCorners, TangentOptions);
					}
				}

				// At this point the mesh will have valid tangents.
				check(RawMesh.WedgeTangentX.Num() == NumWedges);
				check(RawMesh.WedgeTangentY.Num() == NumWedges);
				check(RawMesh.WedgeTangentZ.Num() == NumWedges);

				// Generate lightmap UVs
				if (SrcModel.BuildSettings.bGenerateLightmapUVs)
				{
					if (RawMesh.WedgeTexCoords[SrcModel.BuildSettings.SrcLightmapIndex].Num() == 0)
					{
						SrcModel.BuildSettings.SrcLightmapIndex = 0;
					}

					FLayoutUVRawMeshView RawMeshView(RawMesh, SrcModel.BuildSettings.SrcLightmapIndex, SrcModel.BuildSettings.DstLightmapIndex);
					FLayoutUV Packer(RawMeshView);
					Packer.SetVersion(LightmapUVVersion);

					Packer.FindCharts(OverlappingCorners);

					int32 EffectiveMinLightmapResolution = SrcModel.BuildSettings.MinLightmapResolution;
					if (LightmapUVVersion >= ELightmapUVVersion::ConsiderLightmapPadding)
					{
						if (GLightmassDebugOptions.bPadMappings)
						{
							EffectiveMinLightmapResolution -= 2;
						}
					}

					bool bPackSuccess = Packer.FindBestPacking(EffectiveMinLightmapResolution);
					if (bPackSuccess)
					{
						Packer.CommitPackedUVs();
					}
				}
				HasRawMesh[LODIndex] = true;
			}
			else if (LODIndex > 0 && MeshReduction)
			{
				// If a raw mesh is not explicitly provided, use the raw mesh of the
				// next highest LOD.
				int32 BaseRawMeshIndex = LODIndex - 1;
				RawMesh = LODMeshes[BaseRawMeshIndex];
				OverlappingCorners = LODOverlappingCorners[BaseRawMeshIndex];
				LODBuildSettings[LODIndex] = LODBuildSettings[BaseRawMeshIndex];
				HasRawMesh[LODIndex] = false;
				//Make sure the SectionInfoMap is taken from the Base RawMesh
				int32 SectionNumber = StaticMesh->GetOriginalSectionInfoMap().GetSectionNumber(BaseRawMeshIndex);
				for (int32 SectionIndex = 0; SectionIndex < SectionNumber; ++SectionIndex)
				{
					FMeshSectionInfo Info = StaticMesh->GetOriginalSectionInfoMap().Get(BaseRawMeshIndex, SectionIndex);
					StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
					StaticMesh->GetOriginalSectionInfoMap().Set(LODIndex, SectionIndex, Info);
				}
			}
		}
		check(LODMeshes.Num() == NumSourceModels);
		check(LODOverlappingCorners.Num() == NumSourceModels);

		// Bail if there is no raw mesh data from which to build a renderable mesh.
		if (LODMeshes.Num() == 0)
		{
			UE_LOGF(LogMeshUtilities, Error, "Raw Mesh data contains no mesh data to build a mesh that can be rendered.");
			return false;
		}
		else if (LODMeshes[0].WedgeIndices.Num() == 0)
		{
			UE_LOGF(LogMeshUtilities, Error, "Raw Mesh data contains no wedge index data to build a mesh that can be rendered.");
			return false;
		}

		Stage = EStage::Gathered;
		return true;
	}

	bool ReduceLODs(const FStaticMeshLODGroup& LODGroup, IMeshReduction* MeshReduction, TArray<bool>& OutWasReduced)
	{
		check(Stage == EStage::Gathered);
		check(StaticMesh != nullptr);
		int32 NumSourceModels = StaticMesh->GetNumSourceModels();
		if (NumSourceModels == 0)
		{
			UE_LOGF(LogMeshUtilities, Error, "Mesh contains zero source models.");
			return false;
		}

		FMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<FMeshUtilities>("MeshUtilities");

		// Reduce each LOD mesh according to its reduction settings.
		for (int32 LODIndex = 0; LODIndex < NumSourceModels; ++LODIndex)
		{
			const FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
			FMeshReductionSettings ReductionSettings = LODGroup.GetSettings(SrcModel.ReductionSettings, LODIndex);
			LODMaxDeviation[NumValidLODs] = 0.0f;
			if (LODIndex != NumValidLODs)
			{
				LODBuildSettings[NumValidLODs] = LODBuildSettings[LODIndex];
				LODOverlappingCorners[NumValidLODs] = LODOverlappingCorners[LODIndex];
			}

			if (MeshReduction && (ReductionSettings.PercentTriangles < 1.0f || ReductionSettings.MaxDeviation > 0.0f))
			{
				FRawMesh& InMesh = LODMeshes[ReductionSettings.BaseLODModel];
				FRawMesh& DestMesh = LODMeshes[NumValidLODs];
				FOverlappingCorners& InOverlappingCorners = LODOverlappingCorners[ReductionSettings.BaseLODModel];
				FOverlappingCorners& DestOverlappingCorners = LODOverlappingCorners[NumValidLODs];

				FMeshDescription SrcMeshdescription;
				FStaticMeshAttributes(SrcMeshdescription).Register();

				FMeshDescription DestMeshdescription;
				FStaticMeshAttributes(DestMeshdescription).Register();

				TMap<int32, FName> FromMaterialMap;
				FStaticMeshOperations::ConvertFromRawMesh(InMesh, SrcMeshdescription, FromMaterialMap);
				MeshReduction->ReduceMeshDescription(DestMeshdescription, LODMaxDeviation[NumValidLODs], SrcMeshdescription, InOverlappingCorners, ReductionSettings);
				TMap<FName, int32> ToMaterialMap;
				FStaticMeshOperations::ConvertToRawMesh(DestMeshdescription, DestMesh, ToMaterialMap);

				if (DestMesh.WedgeIndices.Num() > 0 && !DestMesh.IsValid())
				{
					UE_LOGF(LogMeshUtilities, Error, "Mesh reduction produced a corrupt mesh for LOD%d", LODIndex);
					return false;
				}
				OutWasReduced[LODIndex] = true;

				// Recompute adjacency information.
				float ComparisonThreshold = GetComparisonThreshold(LODBuildSettings[NumValidLODs]);
				MeshUtilities.FindOverlappingCorners(DestOverlappingCorners, DestMesh, ComparisonThreshold);

				//Make sure the static mesh SectionInfoMap is up to date with the new reduce LOD
				//We have to remap the material index with the ReductionSettings.BaseLODModel sectionInfoMap
				if (StaticMesh != nullptr)
				{
					if (DestMesh.IsValid())
					{
						//Set the new SectionInfoMap for this reduced LOD base on the ReductionSettings.BaseLODModel SectionInfoMap
						const FMeshSectionInfoMap& BaseLODModelSectionInfoMap = StaticMesh->GetSectionInfoMap();
						TArray<int32> UniqueMaterialIndex;
						//Find all unique Material in used order
						int32 NumFaces = DestMesh.FaceMaterialIndices.Num();
						for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
						{
							int32 MaterialIndex = DestMesh.FaceMaterialIndices[FaceIndex];
							UniqueMaterialIndex.AddUnique(MaterialIndex);
						}
						//All used material represent a different section
						for (int32 SectionIndex = 0; SectionIndex < UniqueMaterialIndex.Num(); ++SectionIndex)
						{
							//Section material index have to be remap with the ReductionSettings.BaseLODModel SectionInfoMap to create
							//a valid new section info map for the reduced LOD.
							if (BaseLODModelSectionInfoMap.IsValidSection(ReductionSettings.BaseLODModel, UniqueMaterialIndex[SectionIndex]))
							{
								FMeshSectionInfo SectionInfo = BaseLODModelSectionInfoMap.Get(ReductionSettings.BaseLODModel, UniqueMaterialIndex[SectionIndex]);
								//Try to recuperate the valid data
								if (BaseLODModelSectionInfoMap.IsValidSection(LODIndex, SectionIndex))
								{
									//If the old LOD section was using the same Material copy the data
									FMeshSectionInfo OriginalLODSectionInfo = BaseLODModelSectionInfoMap.Get(LODIndex, SectionIndex);
									if (OriginalLODSectionInfo.MaterialIndex == SectionInfo.MaterialIndex)
									{
										SectionInfo.bCastShadow = OriginalLODSectionInfo.bCastShadow;
										SectionInfo.bEnableCollision = OriginalLODSectionInfo.bEnableCollision;
									}
								}
								//Copy the BaseLODModel section info to the reduce LODIndex.
								StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);
							}
						}
					}
				}
			}

			if (LODMeshes[NumValidLODs].WedgeIndices.Num() > 0)
			{
				NumValidLODs++;
			}
		}

		if (NumValidLODs < 1)
		{
			UE_LOGF(LogMeshUtilities, Error, "Mesh reduction produced zero LODs.");
			return false;
		}
		Stage = EStage::Reduce;
		return true;
	}

	bool GenerateRenderingMeshes(FMeshUtilities& MeshUtilities, FStaticMeshRenderData& OutRenderData)
	{
		check(Stage == EStage::Reduce);
		check(StaticMesh != nullptr);

		int32 ImportVersion = StaticMesh->GetImportVersion();

		// Generate per-LOD rendering data.
		OutRenderData.AllocateLODResources(NumValidLODs);
		for (int32 LODIndex = 0; LODIndex < NumValidLODs; ++LODIndex)
		{
			FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODIndex);
			FStaticMeshLODResources& LODModel = OutRenderData.LODResources[LODIndex];
			FRawMesh& RawMesh = LODMeshes[LODIndex];
			LODModel.MaxDeviation = LODMaxDeviation[LODIndex];

			TArray<FStaticMeshBuildVertex> Vertices;
			TArray<TArray<uint32> > PerSectionIndices;

			TMap<uint32, uint32> MaterialToSectionMapping;

			// Find out how many sections are in the mesh.
			TArray<int32> MaterialIndices;
			for ( const int32 MaterialIndex : RawMesh.FaceMaterialIndices )
			{
				// Find all unique material indices
				MaterialIndices.AddUnique(MaterialIndex);
			}

			// Need X number of sections for X number of material indices
			//for (const int32 MaterialIndex : MaterialIndices)
			for ( int32 Index = 0; Index < MaterialIndices.Num(); ++Index)
			{
				const int32 MaterialIndex = MaterialIndices[Index];
				FStaticMeshSection* Section = new(LODModel.Sections) FStaticMeshSection();
				Section->MaterialIndex = MaterialIndex;
				if (ImportVersion < RemoveStaticMeshSkinxxWorkflow)
				{
					MaterialToSectionMapping.Add(MaterialIndex, MaterialIndex);
				}
				else
				{
					MaterialToSectionMapping.Add(MaterialIndex, Index);
				}
				new(PerSectionIndices)TArray<uint32>;
			}

			// Build and cache optimize vertex and index buffers.
			{
				// TODO_STATICMESH: The wedge map is only valid for LODIndex 0 if no reduction has been performed.
				// TODO - write directly to TMemoryImageArray
				// We can compute an approximate one instead for other LODs.
				TArray<int32> TempWedgeMap;
				TArray<int32>& WedgeMap = SourceModel.ReductionSettings.PercentTriangles >= 1.0f ? LODModel.WedgeMap : TempWedgeMap;
				WedgeMap.Reset();
				float ComparisonThreshold = GetComparisonThreshold(LODBuildSettings[LODIndex]);
				MeshUtilities.BuildStaticMeshVertexAndIndexBuffers(Vertices, PerSectionIndices, WedgeMap, RawMesh, LODOverlappingCorners[LODIndex], MaterialToSectionMapping, ComparisonThreshold, (FVector3f)LODBuildSettings[LODIndex].BuildScale3D, ImportVersion);
				check(WedgeMap.Num() == RawMesh.WedgeIndices.Num());

				if (RawMesh.WedgeIndices.Num() < 100000 * 3)
				{
					MeshUtilities.CacheOptimizeVertexAndIndexBuffer(Vertices, PerSectionIndices, WedgeMap);
					check(WedgeMap.Num() == RawMesh.WedgeIndices.Num());
				}
			}

			verifyf(Vertices.Num() != 0, TEXT("No valid vertices found for the mesh."));

			// Initialize the vertex buffer.
			int32 NumTexCoords = ComputeNumTexCoords(RawMesh, MAX_STATIC_TEXCOORDS);
			LODModel.VertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(LODBuildSettings[LODIndex].bUseHighPrecisionTangentBasis);
			LODModel.VertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(LODBuildSettings[LODIndex].bUseFullPrecisionUVs);
			FStaticMeshVertexBufferFlags StaticMeshVertexBufferFlags;
			StaticMeshVertexBufferFlags.bNeedsCPUAccess = true;
			StaticMeshVertexBufferFlags.bUseBackwardsCompatibleF16TruncUVs = LODBuildSettings[LODIndex].bUseBackwardsCompatibleF16TruncUVs;
			LODModel.VertexBuffers.StaticMeshVertexBuffer.Init(Vertices, NumTexCoords, StaticMeshVertexBufferFlags);
			LODModel.VertexBuffers.PositionVertexBuffer.Init(Vertices);
			LODModel.VertexBuffers.ColorVertexBuffer.Init(Vertices);

			// Concatenate the per-section index buffers.
			TArray<uint32> CombinedIndices;
			bool bNeeds32BitIndices = false;
			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
				TArray<uint32> const& SectionIndices = PerSectionIndices[SectionIndex];
				Section.FirstIndex = 0;
				Section.NumTriangles = 0;
				Section.MinVertexIndex = 0;
				Section.MaxVertexIndex = 0;

				if (SectionIndices.Num())
				{
					Section.FirstIndex = CombinedIndices.Num();
					Section.NumTriangles = SectionIndices.Num() / 3;

					CombinedIndices.AddUninitialized(SectionIndices.Num());
					uint32* DestPtr = &CombinedIndices[Section.FirstIndex];
					uint32 const* SrcPtr = SectionIndices.GetData();

					Section.MinVertexIndex = *SrcPtr;
					Section.MaxVertexIndex = *SrcPtr;

					for (int32 Index = 0; Index < SectionIndices.Num(); Index++)
					{
						uint32 VertIndex = *SrcPtr++;

						bNeeds32BitIndices |= (VertIndex > MAX_uint16);
						Section.MinVertexIndex = FMath::Min<uint32>(VertIndex, Section.MinVertexIndex);
						Section.MaxVertexIndex = FMath::Max<uint32>(VertIndex, Section.MaxVertexIndex);
						*DestPtr++ = VertIndex;
					}
				}
			}
			LODModel.IndexBuffer.SetIndices(CombinedIndices, bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit);
			
			// Build the reversed index buffer.
			if (LODModel.AdditionalIndexBuffers && StaticMesh->GetSourceModel(0).BuildSettings.bBuildReversedIndexBuffer)
			{
				TArray<uint32> InversedIndices;
				const int32 IndexCount = CombinedIndices.Num();
				InversedIndices.AddUninitialized(IndexCount);

				for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
				{
					const FStaticMeshSection& SectionInfo = LODModel.Sections[SectionIndex];
					const int32 SectionIndexCount = SectionInfo.NumTriangles * 3;

					for (int32 i = 0; i < SectionIndexCount; ++i)
					{
						InversedIndices[SectionInfo.FirstIndex + i] = CombinedIndices[SectionInfo.FirstIndex + SectionIndexCount - 1 - i];
					}
				}
				LODModel.AdditionalIndexBuffers->ReversedIndexBuffer.SetIndices(InversedIndices, bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit);
			}

			// Build the depth-only index buffer.
			TArray<uint32> DepthOnlyIndices;
			{
				BuildDepthOnlyIndexBuffer(
					DepthOnlyIndices,
					Vertices,
					CombinedIndices,
					LODModel.Sections
				);

				if (DepthOnlyIndices.Num() < 50000 * 3)
				{
					MeshUtilities.CacheOptimizeIndexBuffer(DepthOnlyIndices);
				}

				LODModel.DepthOnlyIndexBuffer.SetIndices(DepthOnlyIndices, bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit);
			}

			// Build the inversed depth only index buffer.
			if (LODModel.AdditionalIndexBuffers && StaticMesh->GetSourceModel(0).BuildSettings.bBuildReversedIndexBuffer)
			{
				TArray<uint32> ReversedDepthOnlyIndices;
				const int32 IndexCount = DepthOnlyIndices.Num();
				ReversedDepthOnlyIndices.AddUninitialized(IndexCount);
				for (int32 i = 0; i < IndexCount; ++i)
				{
					ReversedDepthOnlyIndices[i] = DepthOnlyIndices[IndexCount - 1 - i];
				}
				LODModel.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.SetIndices(ReversedDepthOnlyIndices, bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit);
			}

			// Build a list of wireframe edges in the static mesh.
			if (LODModel.AdditionalIndexBuffers)
			{
				TArray<FMeshEdgeDef> Edges;
				TArray<uint32> WireframeIndices;

				FStaticMeshEdgeBuilder(CombinedIndices, Vertices, Edges).FindEdges();
				WireframeIndices.Empty(2 * Edges.Num());
				for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); EdgeIndex++)
				{
					FMeshEdgeDef&	Edge = Edges[EdgeIndex];
					WireframeIndices.Add(Edge.Vertices[0]);
					WireframeIndices.Add(Edge.Vertices[1]);
				}
				LODModel.AdditionalIndexBuffers->WireframeIndexBuffer.SetIndices(WireframeIndices, bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit);
			}
		}

		// Copy the original material indices to fixup meshes before compacting of materials was done.
		if (NumValidLODs > 0)
		{
			OutRenderData.MaterialIndexToImportIndex = LODMeshes[0].MaterialIndexToImportIndex;
		}

		// Calculate the bounding box.
		FBounds3f Bounds;
		FPositionVertexBuffer& BasePositionVertexBuffer = OutRenderData.LODResources[0].VertexBuffers.PositionVertexBuffer;
		for (uint32 VertexIndex = 0; VertexIndex < BasePositionVertexBuffer.GetNumVertices(); VertexIndex++)
		{
			Bounds += BasePositionVertexBuffer.VertexPosition(VertexIndex);
		}
		OutRenderData.Bounds.Origin = (FVector)Bounds.GetCenter();
		OutRenderData.Bounds.BoxExtent = (FVector)Bounds.GetExtent();

		// Calculate the bounding sphere, using the center of the bounding box as the origin.
		OutRenderData.Bounds.SphereRadius = 0.0f;
		for (uint32 VertexIndex = 0; VertexIndex < BasePositionVertexBuffer.GetNumVertices(); VertexIndex++)
		{
			OutRenderData.Bounds.SphereRadius = FMath::Max(
				(BasePositionVertexBuffer.VertexPosition(VertexIndex) - (FVector3f)OutRenderData.Bounds.Origin).SizeSquared(),
				OutRenderData.Bounds.SphereRadius
				);
		}
		OutRenderData.Bounds.SphereRadius = FMath::Sqrt( OutRenderData.Bounds.SphereRadius );

		Stage = EStage::GenerateRendering;
		return true;
	}

	bool ReplaceRawMeshModels()
	{
		check(Stage == EStage::Reduce);
		check(StaticMesh != nullptr);

		check(HasRawMesh[0]);
		check(StaticMesh->GetNumSourceModels() >= NumValidLODs);
		bool bDirty = false;
		for (int32 Index = 1; Index < NumValidLODs; ++Index)
		{
			if (!HasRawMesh[Index])
			{
				StaticMesh->GetSourceModel(Index).SaveRawMesh(LODMeshes[Index]);
				bDirty = true;
			}
		}

		Stage = EStage::ReplaceRaw;
		return true;
	}

private:
	enum class EStage
	{
		Uninit,
		Gathered,
		Reduce,
		GenerateRendering,
		ReplaceRaw,
	};

	EStage Stage;

	int32 NumValidLODs;

	TIndirectArray<FRawMesh> LODMeshes;
	TIndirectArray<FOverlappingCorners> LODOverlappingCorners;
	float LODMaxDeviation[MAX_STATIC_MESH_LODS];
	FMeshBuildSettings LODBuildSettings[MAX_STATIC_MESH_LODS];
	bool HasRawMesh[MAX_STATIC_MESH_LODS];
	UStaticMesh* StaticMesh;
};

template<class MaterialSlotType>
void FixupMaterialSlotNames_Implementation(TArray<MaterialSlotType>& MaterialSlots)
{
	TArray<FName> UniqueMImportedaterialSlotName;

	//Make sure we have non empty imported material slot names
	for (MaterialSlotType& MaterialSlot : MaterialSlots)
	{
		if (MaterialSlot.ImportedMaterialSlotName == NAME_None)
		{
			if (MaterialSlot.MaterialSlotName != NAME_None)
			{
				MaterialSlot.ImportedMaterialSlotName = MaterialSlot.MaterialSlotName;
			}
			else if (MaterialSlot.MaterialInterface != nullptr)
			{
				MaterialSlot.ImportedMaterialSlotName = MaterialSlot.MaterialInterface->GetFName();
			}
			else
			{
				MaterialSlot.ImportedMaterialSlotName = FName(TEXT("MaterialSlot"));
			}
		}

		FString BaseMaterialSlotName = MaterialSlot.ImportedMaterialSlotName.ToString();
		FString UniqueName = BaseMaterialSlotName;
		int32 UniqueIndex = 1;
		while (UniqueMImportedaterialSlotName.Contains(FName(*UniqueName)))
		{
			UniqueName = FString::Printf(TEXT("%s_%d"), *BaseMaterialSlotName, UniqueIndex);
			UniqueIndex++;
		}
		MaterialSlot.ImportedMaterialSlotName = FName(*UniqueName);
		UniqueMImportedaterialSlotName.Add(MaterialSlot.ImportedMaterialSlotName);
		if (MaterialSlot.MaterialSlotName == NAME_None)
		{
			MaterialSlot.MaterialSlotName = MaterialSlot.ImportedMaterialSlotName;
		}
	}
}

void FMeshUtilities::FixupMaterialSlotNames(UStaticMesh* StaticMesh) const
{
	FixupMaterialSlotNames_Implementation(StaticMesh->GetStaticMaterials());
}

void FMeshUtilities::FixupMaterialSlotNames(USkeletalMesh* SkeletalMesh) const
{
	FixupMaterialSlotNames_Implementation(SkeletalMesh->GetMaterials());
}

bool FMeshUtilities::BuildStaticMesh(FStaticMeshRenderData& OutRenderData, UStaticMesh* StaticMesh, const FStaticMeshLODGroup& LODGroup)
{
	int32 LightmapUVVersion = StaticMesh->GetLightmapUVVersion();
	int32 ImportVersion = StaticMesh->GetImportVersion();

	IMeshReductionManagerModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	FStaticMeshUtilityBuilder Builder(StaticMesh);
	if (!Builder.GatherSourceMeshesPerLOD(Module.GetStaticMeshReductionInterface()))
	{
		return false;
	}

	TArray<bool> WasReduced;
	WasReduced.AddZeroed(StaticMesh->GetNumSourceModels());
	if (!Builder.ReduceLODs(LODGroup, Module.GetStaticMeshReductionInterface(), WasReduced))
	{
		return false;
	}

	return Builder.GenerateRenderingMeshes(*this, OutRenderData);
}

bool FMeshUtilities::GenerateStaticMeshLODs(UStaticMesh* StaticMesh, const FStaticMeshLODGroup& LODGroup)
{
	int32 LightmapUVVersion = StaticMesh->GetLightmapUVVersion();

	FStaticMeshUtilityBuilder Builder(StaticMesh);
	IMeshReductionManagerModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	if (!Builder.GatherSourceMeshesPerLOD(Module.GetStaticMeshReductionInterface()))
	{
		return false;
	}

	TArray<bool> WasReduced;
	WasReduced.AddZeroed(StaticMesh->GetNumSourceModels());
	if (!Builder.ReduceLODs(LODGroup, Module.GetStaticMeshReductionInterface(), WasReduced))
	{
		return false;
	}

	if (WasReduced.Contains(true))
	{
		return Builder.ReplaceRawMeshModels();
	}

	return false;
}

class SkeletalMeshBuildData
{
public:
	SkeletalMeshBuildData(
		FSkeletalMeshLODModel& InLODModel,
		const FReferenceSkeleton& InRefSkeleton,
		const TArray<SkeletalMeshImportData::FVertInfluence>& InInfluences,
		const TArray<SkeletalMeshImportData::FMeshWedge>& InWedges,
		const TArray<SkeletalMeshImportData::FMeshFace>& InFaces,
		const TArray<FVector3f>& InPoints,
		const TArray<int32>& InPointToOriginalMap,
		const IMeshUtilities::MeshBuildOptions& InBuildOptions,
		TArray<FText>* InWarningMessages,
		TArray<FName>* InWarningNames)
		: MikkTUserData(InWedges, InFaces, InPoints, InBuildOptions.bComputeNormals, TangentX, TangentY, TangentZ)
		, LODModel(InLODModel)
		, RefSkeleton(InRefSkeleton)
		, Influences(InInfluences)
		, Wedges(InWedges)
		, Faces(InFaces)
		, Points(InPoints)
		, PointToOriginalMap(InPointToOriginalMap)
		, BuildOptions(InBuildOptions)
		, OutWarningMessages(InWarningMessages)
		, OutWarningNames(InWarningNames)
		, bTooManyVerts(false)
	{
		MikkTInterface.m_getNormal = MikkGetNormal_Skeletal;
		MikkTInterface.m_getNumFaces = MikkGetNumFaces_Skeletal;
		MikkTInterface.m_getNumVerticesOfFace = MikkGetNumVertsOfFace_Skeletal;
		MikkTInterface.m_getPosition = MikkGetPosition_Skeletal;
		MikkTInterface.m_getTexCoord = MikkGetTexCoord_Skeletal;
		MikkTInterface.m_setTSpaceBasic = MikkSetTSpaceBasic_Skeletal;
		MikkTInterface.m_setTSpace = nullptr;

		//Fill the NTBs information
		if (!InBuildOptions.bComputeNormals || !InBuildOptions.bComputeTangents)
		{
			if (!InBuildOptions.bComputeTangents)
			{
				TangentX.AddZeroed(Wedges.Num());
				TangentY.AddZeroed(Wedges.Num());
			}

			if (!InBuildOptions.bComputeNormals)
			{
				TangentZ.AddZeroed(Wedges.Num());
			}

			// TODO: Does Nanite even use this data? Likely only with explicit tangents!
			for (const SkeletalMeshImportData::FMeshFace& MeshFace : Faces)
			{
				for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
				{
					uint32 WedgeIndex = MeshFace.iWedge[CornerIndex];
					if (!InBuildOptions.bComputeTangents)
					{
						TangentX[WedgeIndex] = MeshFace.TangentX[CornerIndex];
						TangentY[WedgeIndex] = MeshFace.TangentY[CornerIndex];
					}
					if (!InBuildOptions.bComputeNormals)
					{
						TangentZ[WedgeIndex] = MeshFace.TangentZ[CornerIndex];
					}
				}
			}
		}
	}

	inline uint32 GetWedgeIndex(uint32 FaceIndex, uint32 TriIndex) const
	{
		return Faces[FaceIndex].iWedge[TriIndex];
	}

	inline uint32 GetVertexIndex(uint32 WedgeIndex) const
	{
		return Wedges[WedgeIndex].iVertex;
	}

	inline uint32 GetVertexIndex(uint32 FaceIndex, uint32 TriIndex) const
	{
		return Wedges[Faces[FaceIndex].iWedge[TriIndex]].iVertex;
	}

	inline const FVector3f& GetVertexPosition(uint32 WedgeIndex) const
	{
		return Points[Wedges[WedgeIndex].iVertex];
	}

	inline const FVector3f& GetVertexPosition(uint32 FaceIndex, uint32 TriIndex) const
	{
		return Points[Wedges[Faces[FaceIndex].iWedge[TriIndex]].iVertex];
	}

	inline const FVector2f& GetVertexUV(uint32 FaceIndex, uint32 TriIndex, uint32 UVIndex) const
	{
		return Wedges[Faces[FaceIndex].iWedge[TriIndex]].UVs[UVIndex];
	}

	inline uint32 GetFaceSmoothingGroups(uint32 FaceIndex) const
	{
		return Faces[FaceIndex].SmoothingGroups;
	}

	inline uint32 GetNumFaces() const
	{
		return Faces.Num();
	}

	inline uint32 GetNumWedges() const
	{
		return Wedges.Num();
	}

	inline TArray<FVector3f>& GetTangentArray(uint32 Axis)
	{
		if (Axis == 0)
		{
			return TangentX;
		}
		else if (Axis == 1)
		{
			return TangentY;
		}

		return TangentZ;
	}

	inline void ValidateTangentArraySize() const
	{
		check(TangentX.Num() == Wedges.Num());
		check(TangentY.Num() == Wedges.Num());
		check(TangentZ.Num() == Wedges.Num());
	}

	inline SMikkTSpaceInterface* GetMikkTInterface()
	{
		return &MikkTInterface;
	}

	inline void* GetMikkTUserData()
	{
		return (void*)&MikkTUserData;
	}

	TArray<FVector3f> TangentX;
	TArray<FVector3f> TangentY;
	TArray<FVector3f> TangentZ;
	TArray<FSkinnedMeshChunk*> Chunks;

	SMikkTSpaceInterface MikkTInterface;
	MikkTSpace_Skeletal_Mesh MikkTUserData;

	FSkeletalMeshLODModel& LODModel;
	const FReferenceSkeleton& RefSkeleton;
	const TArray<SkeletalMeshImportData::FVertInfluence>& Influences;
	const TArray<SkeletalMeshImportData::FMeshWedge>& Wedges;
	const TArray<SkeletalMeshImportData::FMeshFace>& Faces;
	const TArray<FVector3f>& Points;
	const TArray<int32>& PointToOriginalMap;

	const IMeshUtilities::MeshBuildOptions& BuildOptions;
	TArray<FText>* OutWarningMessages;
	TArray<FName>* OutWarningNames;
	bool bTooManyVerts;
};

class FSkeletalMeshUtilityBuilder
{
public:
	FSkeletalMeshUtilityBuilder()
		: Stage(EStage::Uninit)
	{
	}

public:
	void Skeletal_FindOverlappingCorners(
		FOverlappingCorners& OutOverlappingCorners,
		SkeletalMeshBuildData& BuildData,
		float ComparisonThreshold
		)
	{
		int32 NumFaces = BuildData.GetNumFaces();
		int32 NumWedges = BuildData.GetNumWedges();
		check(NumFaces * 3 <= NumWedges);

		// Create a list of vertex Z/index pairs
		TArray<FIndexAndZ> VertIndexAndZ;
		VertIndexAndZ.Empty(NumWedges);
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
		{
			for (int32 TriIndex = 0; TriIndex < 3; ++TriIndex)
			{
				uint32 Index = BuildData.GetWedgeIndex(FaceIndex, TriIndex);
				new(VertIndexAndZ)FIndexAndZ(Index, BuildData.GetVertexPosition(Index));
			}
		}

		// Sort the vertices by z value
		VertIndexAndZ.Sort(FCompareIndexAndZ());

		OutOverlappingCorners.Init(NumWedges);

		// Search for duplicates, quickly!
		for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
		{
			// only need to search forward, since we add pairs both ways
			for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
			{
				if (FMath::Abs(VertIndexAndZ[j].Z - VertIndexAndZ[i].Z) > ComparisonThreshold)
					break; // can't be any more dups

				const FVector3f& PositionA = BuildData.GetVertexPosition(VertIndexAndZ[i].Index);
				const FVector3f& PositionB = BuildData.GetVertexPosition(VertIndexAndZ[j].Index);

				if (PointsEqual(PositionA, PositionB, ComparisonThreshold))
				{
					OutOverlappingCorners.Add(VertIndexAndZ[i].Index, VertIndexAndZ[j].Index);
				}
			}
		}

		OutOverlappingCorners.FinishAdding();
	}

	void Skeletal_ComputeTriangleTangents(
		TArray<FVector3f>& TriangleTangentX,
		TArray<FVector3f>& TriangleTangentY,
		TArray<FVector3f>& TriangleTangentZ,
		SkeletalMeshBuildData& BuildData,
		float ComparisonThreshold
		)
	{
		int32 NumTriangles = BuildData.GetNumFaces();
		TriangleTangentX.Reset(NumTriangles);
		TriangleTangentY.Reset(NumTriangles);
		TriangleTangentZ.Reset(NumTriangles);

		//Currently GetSafeNormal do not support 0.0f threshold properly
		float RealComparisonThreshold = FMath::Max(ComparisonThreshold, FLT_MIN);
		
		TArray<FVector3f> TriangleTangents;
		TriangleTangents.Reserve(NumTriangles * 3);

		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
		{
			const FVector3f& Position0 = BuildData.GetVertexPosition(TriangleIndex, 0);
			const FVector3f& Position1 = BuildData.GetVertexPosition(TriangleIndex, 1);
			const FVector3f& Position2 = BuildData.GetVertexPosition(TriangleIndex, 2);

			const int32 UVIndex = 0;
			const FVector2f& UV0 = FVector2f(BuildData.GetVertexUV(TriangleIndex, 0, UVIndex));
			const FVector2f& UV1 = FVector2f(BuildData.GetVertexUV(TriangleIndex, 1, UVIndex));
			const FVector2f& UV2 = FVector2f(BuildData.GetVertexUV(TriangleIndex, 2, UVIndex));

			CalculateTriangleTangentInternal(Position0, UV0, Position1, UV1, Position2, UV2, TriangleTangents, RealComparisonThreshold);
			checkSlow(TriangleTangents.Num() == 3);

			TriangleTangentX.Add(TriangleTangents[0]);
			TriangleTangentY.Add(TriangleTangents[1]);
			TriangleTangentZ.Add(TriangleTangents[2]);
		}
	}

	struct FAdjacentFaceContext
	{
		TMap<int32, int32> AdjacentFaceCommonVertices;
	};

	//This function add every triangles connected to the triangle queue.
	//A connected triangle pair must share at least 1 vertex between the two triangles.
	//If bConnectByEdge is true, the connected triangle must share at least one edge (two vertex index)
	void AddAdjacentFace(SkeletalMeshBuildData& BuildData, TBitArray<>& FaceAdded, TMap<int32, TArray<int32>>& VertexIndexToAdjacentFaces, int32 FaceIndex, TArray<int32>& TriangleQueue, FAdjacentFaceContext& Context, const bool bConnectByEdge)
	{
		checkSlow(FaceAdded.Num() == (int32)BuildData.GetNumFaces());

		Context.AdjacentFaceCommonVertices.Reset();

		for (int32 Corner = 0; Corner < 3; Corner++)
		{
			int32 VertexIndex = BuildData.GetVertexIndex(FaceIndex, Corner);
			TArray<int32>& AdjacentFaces = VertexIndexToAdjacentFaces.FindChecked(VertexIndex);
			for (int32 AdjacentFaceArrayIndex = 0; AdjacentFaceArrayIndex < AdjacentFaces.Num(); ++AdjacentFaceArrayIndex)
			{
				int32 AdjacentFaceIndex = AdjacentFaces[AdjacentFaceArrayIndex];
				if (!FaceAdded[AdjacentFaceIndex] && AdjacentFaceIndex != FaceIndex)
				{
					bool bAddConnected = !bConnectByEdge;
					if (bConnectByEdge)
					{
						int32& AdjacentFaceCommonVerticeCount = Context.AdjacentFaceCommonVertices.FindOrAdd(AdjacentFaceIndex);
						AdjacentFaceCommonVerticeCount++;
						//Is the connected triangles share 2 vertex index (one edge) not only one vertex
						bAddConnected = AdjacentFaceCommonVerticeCount > 1;
					}

					if (bAddConnected)
					{
						TriangleQueue.Add(AdjacentFaceIndex);
						//Add the face only once by marking the face has computed
						FaceAdded[AdjacentFaceIndex] = true;
					}
				}
			}
		}
	}

	//Fill FaceIndexToPatchIndex so every triangle know is unique island patch index.
	//We need to respect the island when we use the smooth group to compute the normals.
	//Each island patch have its own smoothgroup data, there is no triangle connectivity possible between island patch.
	//@Param bConnectByEdge: If true we need at least 2 vertex index (one edge) to connect 2 triangle. If false we just need one vertex index (bowtie)
	void Skeletal_FillPolygonPatch(SkeletalMeshBuildData& BuildData, TArray<int32>& FaceIndexToPatchIndex, const bool bConnectByEdge)
	{
		int32 NumTriangles = BuildData.GetNumFaces();
		check(FaceIndexToPatchIndex.Num() == NumTriangles);
		
		int32 PatchIndex = 0;

		TMap<int32, TArray<int32>> VertexIndexToAdjacentFaces;
		VertexIndexToAdjacentFaces.Reserve(NumTriangles*2);
		for (int32 FaceIndex = 0; FaceIndex < NumTriangles; ++FaceIndex)
		{
			int32 WedgeOffset = FaceIndex * 3;
			for (int32 Corner = 0; Corner < 3; Corner++)
			{
				int32 VertexIndex = BuildData.GetVertexIndex(FaceIndex, Corner);
				TArray<int32>& AdjacentFaces = VertexIndexToAdjacentFaces.FindOrAdd(VertexIndex);
				AdjacentFaces.AddUnique(FaceIndex);
			}
		}

		//Mark added face so we do not add them more then once
		TBitArray<> FaceAdded;
		FaceAdded.Init(false, NumTriangles);

		TArray<int32> TriangleQueue;
		TriangleQueue.Reserve(100);

		FAdjacentFaceContext FaceContext;
		FaceContext.AdjacentFaceCommonVertices.Reserve(100);

		for (int32 FaceIndex = 0; FaceIndex < NumTriangles; ++FaceIndex)
		{
			if (FaceAdded[FaceIndex])
			{
				continue;
			}

			TriangleQueue.Reset();
			TriangleQueue.Add(FaceIndex); //Use a queue to avoid recursive function
			FaceAdded[FaceIndex] = true;
			while (TriangleQueue.Num() > 0)
			{
				int32 CurrentTriangleIndex = TriangleQueue.Pop(EAllowShrinking::No);
				FaceIndexToPatchIndex[CurrentTriangleIndex] = PatchIndex;

				AddAdjacentFace(BuildData, FaceAdded, VertexIndexToAdjacentFaces, CurrentTriangleIndex, TriangleQueue, FaceContext, bConnectByEdge);
			}
			PatchIndex++;
		}
	}

	bool IsTriangleMirror(SkeletalMeshBuildData& BuildData, const TArray<FVector3f>& TriangleTangentZ, const uint32 FaceIdxA, const uint32 FaceIdxB)
	{
		if (FaceIdxA == FaceIdxB)
		{
			return false;
		}
		for (int32 CornerA = 0; CornerA < 3; ++CornerA)
		{
			const FVector3f& CornerAPosition = BuildData.GetVertexPosition((FaceIdxA * 3) + CornerA);
			bool bFoundMatch = false;
			for (int32 CornerB = 0; CornerB < 3; ++CornerB)
			{
				const FVector3f& CornerBPosition = BuildData.GetVertexPosition((FaceIdxB * 3) + CornerB);
				if (PointsEqual(CornerAPosition, CornerBPosition, BuildData.BuildOptions.OverlappingThresholds))
				{
					bFoundMatch = true;
					break;
				}
			}

			if (!bFoundMatch)
			{
				return false;
			}
		}
		//Check if the triangles normals are opposite and parallel. Dot product equal -1.0f
		if (FMath::IsNearlyEqual(FVector3f::DotProduct(TriangleTangentZ[FaceIdxA], TriangleTangentZ[FaceIdxB]), -1.0f, KINDA_SMALL_NUMBER))
		{
			return true;
		}
		return false;
	}

	void Skeletal_ComputeTangents(
		const FString& SkeletalMeshName,
		SkeletalMeshBuildData& BuildData,
		const FOverlappingCorners& OverlappingCorners
		)
	{
		bool bBlendOverlappingNormals = true;
		bool bIgnoreDegenerateTriangles = BuildData.BuildOptions.bRemoveDegenerateTriangles;
		bool bComputeWeightedNormals = BuildData.BuildOptions.bComputeWeightedNormals;
		bool bUseMikktSpace = BuildData.BuildOptions.bUseMikkTSpace && (BuildData.BuildOptions.bComputeNormals || BuildData.BuildOptions.bComputeTangents);
		
		int32 NumFaces = BuildData.GetNumFaces();
		int32 NumWedges = BuildData.GetNumWedges();
		check(NumFaces * 3 <= NumWedges);
		
		// Compute per-triangle tangents.
		TArray<FVector3f> TriangleTangentX;
		TArray<FVector3f> TriangleTangentY;
		TArray<FVector3f> TriangleTangentZ;

		Skeletal_ComputeTriangleTangents(
			TriangleTangentX,
			TriangleTangentY,
			TriangleTangentZ,
			BuildData,
			bIgnoreDegenerateTriangles ? SMALL_NUMBER : FLT_MIN
			);

		TArray<int32> FaceIndexToPatchIndex;
		FaceIndexToPatchIndex.AddZeroed(NumFaces);
		//Since we use triangle normals to compute the vertex normal, we need a full edge connected (2 vertex component per triangle)
		const bool bConnectByEdge = true;
		Skeletal_FillPolygonPatch(BuildData, FaceIndexToPatchIndex, bConnectByEdge);

		TArray<FVector3f>& WedgeTangentX = BuildData.GetTangentArray(0);
		TArray<FVector3f>& WedgeTangentY = BuildData.GetTangentArray(1);
		TArray<FVector3f>& WedgeTangentZ = BuildData.GetTangentArray(2);

		// Declare these out here to avoid reallocations.
		TArray<FFanFace> RelevantFacesForCorner[3];
		TArray<int32> AdjacentFaces;

		// Allocate storage for tangents and normal if none were provided.
		if (WedgeTangentX.Num() != NumWedges)
		{
			WedgeTangentX.Empty(NumWedges);
			WedgeTangentX.AddZeroed(NumWedges);
		}
		if (WedgeTangentY.Num() != NumWedges)
		{
			WedgeTangentY.Empty(NumWedges);
			WedgeTangentY.AddZeroed(NumWedges);
		}
		if (WedgeTangentZ.Num() != NumWedges)
		{
			WedgeTangentZ.Empty(NumWedges);
			WedgeTangentZ.AddZeroed(NumWedges);
		}

		bool bIsZeroLengthNormalErrorMessageDisplayed = false;
		// we need to calculate normals for MikkTSpace
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
		{
			int32 PatchIndex = FaceIndexToPatchIndex[FaceIndex];
			int32 WedgeOffset = FaceIndex * 3;
			FVector3f CornerPositions[3];
			FVector3f CornerTangentX[3];
			FVector3f CornerTangentY[3];
			FVector3f CornerNormal[3];

			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				CornerTangentX[CornerIndex] = FVector3f::ZeroVector;
				CornerTangentY[CornerIndex] = FVector3f::ZeroVector;
				CornerNormal[CornerIndex] = FVector3f::ZeroVector;
				CornerPositions[CornerIndex] = BuildData.GetVertexPosition(FaceIndex, CornerIndex);
				RelevantFacesForCorner[CornerIndex].Reset();
			}

			// Don't process degenerate triangles.
			if (PointsEqual(CornerPositions[0], CornerPositions[1], BuildData.BuildOptions.OverlappingThresholds) ||
				PointsEqual(CornerPositions[0], CornerPositions[2], BuildData.BuildOptions.OverlappingThresholds) ||
				PointsEqual(CornerPositions[1], CornerPositions[2], BuildData.BuildOptions.OverlappingThresholds))
			{
				continue;
			}

			// No need to process triangles if tangents already exist.
			bool bCornerHasNormal[3] = { 0 };
			bool bCornerHasTangents[3] = { 0 };
			bool bSkipNTB = true;
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				bCornerHasNormal[CornerIndex] = !WedgeTangentZ[WedgeOffset + CornerIndex].IsNearlyZero();
				bCornerHasTangents[CornerIndex] = !WedgeTangentX[WedgeOffset + CornerIndex].IsNearlyZero() && !WedgeTangentY[WedgeOffset + CornerIndex].IsNearlyZero();

				//If we want to compute mikkt we dont check tangents to skip this corner
				if (!bCornerHasNormal[CornerIndex] || (!bUseMikktSpace && !bCornerHasTangents[CornerIndex]))
				{
					bSkipNTB = false;
				}
			}
			if (bSkipNTB)
			{
				continue;
			}

			// Calculate smooth vertex normals.
			float Determinant = FVector3f::Triple(
				TriangleTangentX[FaceIndex],
				TriangleTangentY[FaceIndex],
				TriangleTangentZ[FaceIndex]
			);

			// Start building a list of faces adjacent to this face.
			AdjacentFaces.Reset();
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				int32 ThisCornerIndex = WedgeOffset + CornerIndex;
				const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(ThisCornerIndex);
				if (DupVerts.Num() == 0)
				{
					AdjacentFaces.AddUnique(ThisCornerIndex / 3); // I am a "dup" of myself
				}
				for (int32 k = 0; k < DupVerts.Num(); k++)
				{
					int32 PotentialTriangleIndex = DupVerts[k] / 3;
					//Do not add a triangle that was remove
					if (TriangleTangentZ.IsValidIndex(PotentialTriangleIndex))
					{
						bool bDegeneratedTriangles = TriangleTangentZ[FaceIndex].IsNearlyZero() || TriangleTangentZ[PotentialTriangleIndex].IsNearlyZero();
						//Do not add mirror triangle to the adjacentFaces. Also make sure adjacent triangle is in the same connected triangle patch. Accept connected degenerate triangle
						if ((bDegeneratedTriangles || !IsTriangleMirror(BuildData, TriangleTangentZ, FaceIndex, PotentialTriangleIndex)) && PatchIndex == FaceIndexToPatchIndex[PotentialTriangleIndex])
						{
							AdjacentFaces.AddUnique(PotentialTriangleIndex);
						}
					}
				}
			}

			// We need to sort these here because the criteria for point equality is
			// exact, so we must ensure the exact same order for all dups.
			AdjacentFaces.Sort();

			// Process adjacent faces
			for (int32 AdjacentFaceIndex = 0; AdjacentFaceIndex < AdjacentFaces.Num(); AdjacentFaceIndex++)
			{
				int32 OtherFaceIndex = AdjacentFaces[AdjacentFaceIndex];
				for (int32 OurCornerIndex = 0; OurCornerIndex < 3; OurCornerIndex++)
				{
					if (bCornerHasNormal[OurCornerIndex] && (bUseMikktSpace || bCornerHasTangents[OurCornerIndex]))
						continue;

					FFanFace NewFanFace;
					int32 CommonIndexCount = 0;

					// Check for vertices in common.
					if (FaceIndex == OtherFaceIndex)
					{
						CommonIndexCount = 3;
						NewFanFace.LinkedVertexIndex = OurCornerIndex;
					}
					else
					{
						// Check matching vertices against main vertex .
						for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; OtherCornerIndex++)
						{
							if (PointsEqual(
								CornerPositions[OurCornerIndex],
								BuildData.GetVertexPosition(OtherFaceIndex, OtherCornerIndex),
								BuildData.BuildOptions.OverlappingThresholds
								))
							{
								CommonIndexCount++;
								NewFanFace.LinkedVertexIndex = OtherCornerIndex;
							}
						}
					}

					// Add if connected by at least one point. Smoothing matches are considered later.
					if (CommonIndexCount > 0)
					{
						NewFanFace.FaceIndex = OtherFaceIndex;
						NewFanFace.bFilled = (OtherFaceIndex == FaceIndex); // Starter face for smoothing floodfill.
						NewFanFace.bBlendTangents = NewFanFace.bFilled;
						NewFanFace.bBlendNormals = NewFanFace.bFilled;
						RelevantFacesForCorner[OurCornerIndex].Add(NewFanFace);
					}
				}
			}

			// Find true relevance of faces for a vertex normal by traversing
			// smoothing-group-compatible connected triangle fans around common vertices.
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				if (bCornerHasNormal[CornerIndex] && (bUseMikktSpace || bCornerHasTangents[CornerIndex]))
					continue;

				int32 NewConnections;
				do
				{
					NewConnections = 0;
					for (int32 OtherFaceIdx = 0; OtherFaceIdx < RelevantFacesForCorner[CornerIndex].Num(); OtherFaceIdx++)
					{
						FFanFace& OtherFace = RelevantFacesForCorner[CornerIndex][OtherFaceIdx];
						// The vertex' own face is initially the only face with bFilled == true.
						if (OtherFace.bFilled)
						{
							for (int32 NextFaceIndex = 0; NextFaceIndex < RelevantFacesForCorner[CornerIndex].Num(); NextFaceIndex++)
							{
								FFanFace& NextFace = RelevantFacesForCorner[CornerIndex][NextFaceIndex];
								if (!NextFace.bFilled) // && !NextFace.bBlendTangents)
								{
									if ((NextFaceIndex != OtherFaceIdx)
											&& (BuildData.GetFaceSmoothingGroups(NextFace.FaceIndex) & BuildData.GetFaceSmoothingGroups(OtherFace.FaceIndex)))
									{
										int32 CommonVertices = 0;
										int32 CommonTangentVertices = 0;
										int32 CommonNormalVertices = 0;
										for (int32 OtherCornerIndex = 0; OtherCornerIndex < 3; OtherCornerIndex++)
										{
											for (int32 NextCornerIndex = 0; NextCornerIndex < 3; NextCornerIndex++)
											{
												int32 NextVertexIndex = BuildData.GetVertexIndex(NextFace.FaceIndex, NextCornerIndex);
												int32 OtherVertexIndex = BuildData.GetVertexIndex(OtherFace.FaceIndex, OtherCornerIndex);
												if (PointsEqual(
													BuildData.GetVertexPosition(NextFace.FaceIndex, NextCornerIndex),
													BuildData.GetVertexPosition(OtherFace.FaceIndex, OtherCornerIndex),
													BuildData.BuildOptions.OverlappingThresholds))
												{
													CommonVertices++;
													if (!bUseMikktSpace && UVsEqual(
														BuildData.GetVertexUV(NextFace.FaceIndex, NextCornerIndex, 0),
														BuildData.GetVertexUV(OtherFace.FaceIndex, OtherCornerIndex, 0),
														BuildData.BuildOptions.OverlappingThresholds))
													{
														CommonTangentVertices++;
													}
													if (bBlendOverlappingNormals
														|| NextVertexIndex == OtherVertexIndex)
													{
														CommonNormalVertices++;
													}
												}
											}
										}
										// Flood fill faces with more than one common vertices which must be touching edges.
										if (CommonVertices > 1)
										{
											NextFace.bFilled = true;
											NextFace.bBlendNormals = (CommonNormalVertices > 1);
											NewConnections++;
											// Only blend tangents if there is no UV seam along the edge with this face.
											if (!bUseMikktSpace && OtherFace.bBlendTangents && CommonTangentVertices > 1)
											{
												float OtherDeterminant = FVector3f::Triple(
													TriangleTangentX[NextFace.FaceIndex],
													TriangleTangentY[NextFace.FaceIndex],
													TriangleTangentZ[NextFace.FaceIndex]
												);
												if ((Determinant * OtherDeterminant) > 0.0f)
												{
													NextFace.bBlendTangents = true;
												}
											}
										}
									}
								}
							}
						}
					}
				} 
				while (NewConnections > 0);
			}

			// Vertex normal construction.
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				bool bUseProvidedNormal = bCornerHasNormal[CornerIndex];
				bool bUseProvidedTangents = !bUseMikktSpace && bCornerHasTangents[CornerIndex];
				if (bUseProvidedNormal)
				{
					CornerNormal[CornerIndex] = WedgeTangentZ[WedgeOffset + CornerIndex];
				}
				if (bUseProvidedTangents)
				{
					CornerTangentX[CornerIndex] = WedgeTangentX[WedgeOffset + CornerIndex];
					CornerTangentY[CornerIndex] = WedgeTangentY[WedgeOffset + CornerIndex];
				}

				if (!bUseProvidedNormal || !bUseProvidedTangents)
				{
					for (int32 RelevantFaceIdx = 0; RelevantFaceIdx < RelevantFacesForCorner[CornerIndex].Num(); RelevantFaceIdx++)
					{
						FFanFace const& RelevantFace = RelevantFacesForCorner[CornerIndex][RelevantFaceIdx];
						if (RelevantFace.bFilled)
						{
							int32 OtherFaceIndex = RelevantFace.FaceIndex;
							float CornerWeight = 1.0f;
							if (bComputeWeightedNormals)
							{
								FVector3f OtherFacePoint[3] = { BuildData.GetVertexPosition(OtherFaceIndex, 0), BuildData.GetVertexPosition(OtherFaceIndex, 1), BuildData.GetVertexPosition(OtherFaceIndex, 2) };
								float OtherFaceArea = TriangleUtilities::ComputeTriangleArea(OtherFacePoint[0], OtherFacePoint[1], OtherFacePoint[2]);
								int32 OtherFaceCornerIndex = RelevantFace.LinkedVertexIndex;
								float OtherFaceAngle = TriangleUtilities::ComputeTriangleCornerAngle(OtherFacePoint[OtherFaceCornerIndex], OtherFacePoint[(OtherFaceCornerIndex + 1) % 3], OtherFacePoint[(OtherFaceCornerIndex + 2) % 3]);
								//Get the CornerWeight
								CornerWeight = OtherFaceArea * OtherFaceAngle;
							}
							if (!bUseProvidedTangents && RelevantFace.bBlendTangents)
							{
								CornerTangentX[CornerIndex] += CornerWeight * TriangleTangentX[OtherFaceIndex];
								CornerTangentY[CornerIndex] += CornerWeight * TriangleTangentY[OtherFaceIndex];
							}
							if (!bUseProvidedNormal && RelevantFace.bBlendNormals)
							{
								CornerNormal[CornerIndex] += CornerWeight * TriangleTangentZ[OtherFaceIndex];
							}
						}
					}
					if (!WedgeTangentX[WedgeOffset + CornerIndex].IsNearlyZero())
					{
						CornerTangentX[CornerIndex] = WedgeTangentX[WedgeOffset + CornerIndex];
					}
					if (!WedgeTangentY[WedgeOffset + CornerIndex].IsNearlyZero())
					{
						CornerTangentY[CornerIndex] = WedgeTangentY[WedgeOffset + CornerIndex];
					}
					if (!WedgeTangentZ[WedgeOffset + CornerIndex].IsNearlyZero())
					{
						CornerNormal[CornerIndex] = WedgeTangentZ[WedgeOffset + CornerIndex];
					}
				}
			}

			// Normalization.
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				CornerNormal[CornerIndex].Normalize();
				if (!bUseMikktSpace)
				{
					// If Skeletal_ComputeTriangleTangents somehow didn't manage to create any tangents for this point and its neighbors,
					// just feed the orthonormal basis computation something. 
					if (CornerTangentX[CornerIndex].IsNearlyZero())
					{
						CornerTangentX[CornerIndex] = FVector3f::XAxisVector;
					}
					if (CornerTangentY[CornerIndex].IsNearlyZero())
					{
						CornerTangentY[CornerIndex] = FVector3f::YAxisVector;
					}
					FVector3f::CreateOrthonormalBasis(CornerTangentX[CornerIndex], CornerTangentY[CornerIndex], CornerNormal[CornerIndex]);
				}
			}

			auto VerifyTangentSpace = [&bIsZeroLengthNormalErrorMessageDisplayed, &BuildData, &SkeletalMeshName](FVector3f& NormalizedVector)
			{
				if (NormalizedVector.IsNearlyZero() || NormalizedVector.ContainsNaN())
				{
					NormalizedVector = FVector3f::ZeroVector;
					//We also notify the log that we compute a zero length normals, so the user is aware of it
					if (!bIsZeroLengthNormalErrorMessageDisplayed)
					{
						bIsZeroLengthNormalErrorMessageDisplayed = true;

						// add warning message if available, do a log if not
						FFormatNamedArguments Args;
						Args.Add(TEXT("SkeletalMeshName"), FText::FromString(SkeletalMeshName));
						FText TextMessage = FText::Format(LOCTEXT("Skeletal_ComputeTangents_MikkTSpace_Warning_ZeroLengthNormal", "{SkeletalMeshName} ComputeTangents MikkTSpace function: Compute a zero length normal vector."), Args);
						if (BuildData.OutWarningMessages)
						{
							BuildData.OutWarningMessages->Add(TextMessage);
							if (BuildData.OutWarningNames)
							{
								BuildData.OutWarningNames->Add(FFbxErrors::Generic_Mesh_TangentsComputeError);
							}
						}
						else
						{
							UE_LOGF(LogSkeletalMesh, Display, "%ls", *(TextMessage.ToString()));
						}
					}
				}
			};

			// Copy back to the mesh.
			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				VerifyTangentSpace(CornerNormal[CornerIndex]);
				WedgeTangentZ[WedgeOffset + CornerIndex] = CornerNormal[CornerIndex];

				if (!bUseMikktSpace)
				{
					VerifyTangentSpace(CornerTangentX[CornerIndex]);
					WedgeTangentX[WedgeOffset + CornerIndex] = CornerTangentX[CornerIndex];

					VerifyTangentSpace(CornerTangentY[CornerIndex]);
					WedgeTangentY[WedgeOffset + CornerIndex] = CornerTangentY[CornerIndex];
				}
			}
		}

		if (bUseMikktSpace)
		{
			// we can use mikktspace to calculate the tangents
			SMikkTSpaceContext MikkTContext;
			MikkTContext.m_pInterface = BuildData.GetMikkTInterface();
			MikkTContext.m_pUserData = BuildData.GetMikkTUserData();
			//MikkTContext.m_bIgnoreDegenerates = bIgnoreDegenerateTriangles;

			genTangSpaceDefault(&MikkTContext);
		}

		check(WedgeTangentX.Num() == NumWedges);
		check(WedgeTangentY.Num() == NumWedges);
		check(WedgeTangentZ.Num() == NumWedges);
	}

	bool PrepareSourceMesh(const FString& SkeletalMeshName, SkeletalMeshBuildData& BuildData)
	{
		check(Stage == EStage::Uninit);

		BeginSlowTask();

		FOverlappingCorners& OverlappingCorners = *new FOverlappingCorners;
		LODOverlappingCorners.Add(&OverlappingCorners);

		float ComparisonThreshold = THRESH_POINTS_ARE_SAME;//GetComparisonThreshold(LODBuildSettings[LODIndex]);
		int32 NumWedges = BuildData.GetNumWedges();

		// Find overlapping corners to accelerate adjacency.
		Skeletal_FindOverlappingCorners(OverlappingCorners, BuildData, ComparisonThreshold);

		// Figure out if we should recompute normals and tangents.
		bool bRecomputeNormals = BuildData.BuildOptions.bComputeNormals;
		bool bRecomputeTangents = BuildData.BuildOptions.bComputeTangents;

		// Dump normals and tangents if we are recomputing them.
		if (bRecomputeTangents)
		{
			TArray<FVector3f>& TangentX = BuildData.GetTangentArray(0);
			TArray<FVector3f>& TangentY = BuildData.GetTangentArray(1);

			TangentX.Empty(NumWedges);
			TangentX.AddZeroed(NumWedges);
			TangentY.Empty(NumWedges);
			TangentY.AddZeroed(NumWedges);
		}
		if (bRecomputeNormals)
		{
			TArray<FVector3f>& TangentZ = BuildData.GetTangentArray(2);
			TangentZ.Empty(NumWedges);
			TangentZ.AddZeroed(NumWedges);
		}

		// Compute any missing tangents. MikkTSpace should be use only when the user want to recompute the normals or tangents otherwise should always fallback on builtin tangent
		Skeletal_ComputeTangents(SkeletalMeshName, BuildData, OverlappingCorners);

		// At this point the mesh will have valid tangents.
		BuildData.ValidateTangentArraySize();
		check(LODOverlappingCorners.Num() == 1);

		EndSlowTask();

		Stage = EStage::Prepared;
		return true;
	}

	bool GenerateSkeletalRenderMesh(SkeletalMeshBuildData& BuildData)
	{
		check(Stage == EStage::Prepared);

		BeginSlowTask();

		// Find wedge influences.
		TArray<int32>	WedgeInfluenceIndices; // TODO: Reserve
		TMap<uint32, uint32> VertexIndexToInfluenceIndexMap; // TODO: Reserve

		for (uint32 LookIdx = 0; LookIdx < (uint32)BuildData.Influences.Num(); LookIdx++)
		{
			// Order matters do not allow the map to overwrite an existing value.
			if (!VertexIndexToInfluenceIndexMap.Find(BuildData.Influences[LookIdx].VertIndex))
			{
				VertexIndexToInfluenceIndexMap.Add(BuildData.Influences[LookIdx].VertIndex, LookIdx);
			}
		}

		for (int32 WedgeIndex = 0; WedgeIndex < BuildData.Wedges.Num(); WedgeIndex++)
		{
			uint32* InfluenceIndex = VertexIndexToInfluenceIndexMap.Find(BuildData.Wedges[WedgeIndex].iVertex);

			if (InfluenceIndex)
			{
				WedgeInfluenceIndices.Add(*InfluenceIndex);
			}
			else
			{
				// we have missing influence vert,  we weight to root
				WedgeInfluenceIndices.Add(0);

				// add warning message
				if (BuildData.OutWarningMessages)
				{
					BuildData.OutWarningMessages->Add(FText::Format(FText::FromString("Missing influence on vert {0}. Weighting it to root."), FText::FromString(FString::FromInt(BuildData.Wedges[WedgeIndex].iVertex))));
					if (BuildData.OutWarningNames)
					{
						BuildData.OutWarningNames->Add(FFbxErrors::SkeletalMesh_VertMissingInfluences);
					}
				}
			}
		}

		check(BuildData.Wedges.Num() == WedgeInfluenceIndices.Num());

		TArray<FSkeletalMeshVertIndexAndZ> VertIndexAndZ;
		TArray<FSoftSkinBuildVertex> RawVertices;

		VertIndexAndZ.Empty(BuildData.Faces.Num() * 3);
		RawVertices.Reserve(BuildData.Faces.Num() * 3);

		const int32 InfluenceLimit = FGPUBaseSkinVertexFactory::GetBoneInfluenceLimitForAsset(
			BuildData.BuildOptions.BoneInfluenceLimit, BuildData.BuildOptions.TargetPlatform);

		bool bIsFirstBoneInfluenceTruncation = true;
		for (int32 FaceIndex = 0; FaceIndex < BuildData.Faces.Num(); FaceIndex++)
		{
			// Only update the status progress bar if we are in the game thread and every thousand faces. 
			// Updating status is extremely slow
			if (FaceIndex % 5000 == 0)
			{
				UpdateSlowTask(FaceIndex, BuildData.Faces.Num());
			}

			const SkeletalMeshImportData::FMeshFace& Face = BuildData.Faces[FaceIndex];

			for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
			{
				FSoftSkinBuildVertex Vertex;
				const uint32 WedgeIndex = BuildData.GetWedgeIndex(FaceIndex, VertexIndex);
				const SkeletalMeshImportData::FMeshWedge& Wedge = BuildData.Wedges[WedgeIndex];

				Vertex.Position = BuildData.GetVertexPosition(FaceIndex, VertexIndex);

				FVector3f TangentX, TangentY, TangentZ;
				TangentX = BuildData.TangentX[WedgeIndex].GetSafeNormal();
				TangentY = BuildData.TangentY[WedgeIndex].GetSafeNormal();
				TangentZ = BuildData.TangentZ[WedgeIndex].GetSafeNormal();

				// Normalize overridden tangents.  It's possible for them to import un-normalized.
				TangentX.Normalize();
				TangentY.Normalize();
				TangentZ.Normalize();

				Vertex.TangentX = TangentX;
				Vertex.TangentY = TangentY;
				Vertex.TangentZ = TangentZ;

				for (int32 UVIndex = 0; UVIndex < MAX_TEXCOORDS; ++UVIndex)
				{
					Vertex.UVs[UVIndex] = Wedge.UVs[UVIndex];
				}

				Vertex.Color = Wedge.Color;

				{
					// Count the influences.
					int32 InfIdx = WedgeInfluenceIndices[Face.iWedge[VertexIndex]];
					int32 LookIdx = InfIdx;

					uint32 InfluenceCount = 0;
					while (BuildData.Influences.IsValidIndex(LookIdx) && (BuildData.Influences[LookIdx].VertIndex == Wedge.iVertex))
					{
						InfluenceCount++;
						LookIdx++;
					}

					InfluenceCount = FMath::Min<uint32>(InfluenceCount, MAX_TOTAL_INFLUENCES);

					// Apply bone influence limit from asset
					{
						InfluenceCount = FMath::Min<uint32>(InfluenceCount, InfluenceLimit);
					}

					if (InfluenceCount > EXTRA_BONE_INFLUENCES && !FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(InfluenceCount, BuildData.BuildOptions.TargetPlatform))
					{
						// Add a single warning message, there could be too many
						if (bIsFirstBoneInfluenceTruncation)
						{
							if (BuildData.OutWarningMessages)
							{
								BuildData.OutWarningMessages->Add(FText::FromString(FString::Printf(TEXT("Skeletal mesh of %d bone influences requires unlimited bone influence mode on. Influence truncated to %d."), InfluenceCount, EXTRA_BONE_INFLUENCES)));
								if (BuildData.OutWarningNames)
								{
									BuildData.OutWarningNames->Add(FFbxErrors::SkeletalMesh_TooManyInfluences);
								}
							}

							bIsFirstBoneInfluenceTruncation = false;
						}

						InfluenceCount = EXTRA_BONE_INFLUENCES;
					}

					using namespace UE::AnimationCore;

					// Setup the vertex influences.
					FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
					float InfluenceWeights[MAX_TOTAL_INFLUENCES];

					int32 ActualInfluenceCount = 0; 
					for (uint32 Index = 0; Index < InfluenceCount; Index++)
					{
						const FBoneIndexType BoneIndex = BuildData.Influences[InfIdx + Index].BoneIndex;
						if (BoneIndex >= BuildData.RefSkeleton.GetRawBoneNum())
							continue;
					
						InfluenceBones[ActualInfluenceCount] = BoneIndex;
						InfluenceWeights[ActualInfluenceCount] = (BuildData.Influences[InfIdx + Index].Weight);
						ActualInfluenceCount++;
					}

					const FBoneWeights BoneWeights = FBoneWeights::Create(InfluenceBones, InfluenceWeights, ActualInfluenceCount);
				
					FMemory::Memzero(Vertex.InfluenceBones);
					FMemory::Memzero(Vertex.InfluenceWeights);
				
					if (BoneWeights.Num() == 0)
					{
						Vertex.InfluenceWeights[0] = std::numeric_limits<uint16>::max();
					}
					else
					{
						int32 Index = 0;
						for (FBoneWeight BoneWeight : BoneWeights)
						{
							Vertex.InfluenceBones[Index] = BoneWeight.GetBoneIndex();
							Vertex.InfluenceWeights[Index] = BoneWeight.GetRawWeight();
							Index++;
						}
					}
				}

				// Add the vertex as well as its original index in the points array
				Vertex.PointWedgeIdx = Wedge.iVertex;

				int32 RawIndex = RawVertices.Add(Vertex);

				// Add an efficient way to find dupes of this vertex later for fast combining of vertices
				FSkeletalMeshVertIndexAndZ IAndZ;
				IAndZ.Index = RawIndex;
				const FVector3f& V = Vertex.Position;
				IAndZ.Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;

				VertIndexAndZ.Add(IAndZ);
			}
		}

		// Generate chunks and their vertices and indices
		SkeletalMeshTools::BuildSkeletalMeshChunks(BuildData.Faces, RawVertices, VertIndexAndZ, BuildData.BuildOptions.OverlappingThresholds, BuildData.Chunks, BuildData.bTooManyVerts);

		//Get alternate skinning weights map to retrieve easily the data
		TMap<uint32, TArray<FBoneIndexType>> AlternateBoneIDs;
		AlternateBoneIDs.Reserve(BuildData.Points.Num());
		for (const auto& Kvp : BuildData.LODModel.SkinWeightProfiles)
		{
			const FImportedSkinWeightProfileData& ImportedProfileData = Kvp.Value;
			if (ImportedProfileData.SourceModelInfluences.Num() > 0)
			{
				for (int32 InfluenceIndex = 0; InfluenceIndex < ImportedProfileData.SourceModelInfluences.Num(); ++InfluenceIndex)
				{
					const SkeletalMeshImportData::FVertInfluence& VertInfluence = ImportedProfileData.SourceModelInfluences[InfluenceIndex];
					if (VertInfluence.Weight > 0.0f)
					{
						TArray<FBoneIndexType>& BoneMap = AlternateBoneIDs.FindOrAdd(VertInfluence.VertIndex);
						BoneMap.AddUnique(VertInfluence.BoneIndex);
					}
				}
			}
		}

		SkeletalMeshTools::ChunkSkinnedVertices(BuildData.Chunks, AlternateBoneIDs, FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones);

		EndSlowTask();

		Stage = EStage::GenerateRendering;
		return true;
	}

	void BeginSlowTask()
	{
		if (IsInGameThread())
		{
			GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "ProcessingSkeletalTriangles", "Processing Mesh Triangles"), true);
		}
	}

	void UpdateSlowTask(int32 Numerator, int32 Denominator)
	{
		if (IsInGameThread())
		{
			GWarn->StatusUpdate(Numerator, Denominator, NSLOCTEXT("UnrealEd", "ProcessingSkeletalTriangles", "Processing Mesh Triangles"));
		}
	}

	void EndSlowTask()
	{
		if (IsInGameThread())
		{
			GWarn->EndSlowTask();
		}
	}

private:
	enum class EStage
	{
		Uninit,
		Prepared,
		GenerateRendering,
	};

	TIndirectArray<FOverlappingCorners> LODOverlappingCorners;
	EStage Stage;
};

bool FMeshUtilities::BuildSkeletalMesh(
	FSkeletalMeshLODModel& LODModel,
	const FString& SkeletalMeshName,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<SkeletalMeshImportData::FVertInfluence>& Influences,
	const TArray<SkeletalMeshImportData::FMeshWedge>& Wedges,
	const TArray<SkeletalMeshImportData::FMeshFace>& Faces,
	const TArray<FVector3f>& Points,
	const TArray<int32>& PointToOriginalMap,
	const MeshBuildOptions& BuildOptions,
	TArray<FText>* OutWarningMessages,
	TArray<FName>* OutWarningNames
)
{
#if WITH_EDITORONLY_DATA

	auto UpdateOverlappingVertices = [](FSkeletalMeshLODModel& InLODModel)
	{
		// clear first

		int32 MaxSectionVertices = 0;

		for (int32 SectionIdx = 0; SectionIdx < InLODModel.Sections.Num(); SectionIdx++)
		{
			FSkelMeshSection& CurSection = InLODModel.Sections[SectionIdx];
			CurSection.OverlappingVertices.Reset();

			MaxSectionVertices = FMath::Max(MaxSectionVertices, CurSection.SoftVertices.Num());
		}

		TArray<FIndexAndZ> VertIndexAndZ;
		VertIndexAndZ.Reserve(MaxSectionVertices);

		for (int32 SectionIdx = 0; SectionIdx < InLODModel.Sections.Num(); SectionIdx++)
		{
			FSkelMeshSection& CurSection = InLODModel.Sections[SectionIdx];
			const int32 NumSoftVertices = CurSection.SoftVertices.Num();

			// Create a list of vertex Z/index pairs
			VertIndexAndZ.Reset();
			for (int32 VertIndex = 0; VertIndex < NumSoftVertices; ++VertIndex)
			{
				FSoftSkinVertex& SrcVert = CurSection.SoftVertices[VertIndex];
				new(VertIndexAndZ)FIndexAndZ(VertIndex, SrcVert.Position);
			}
			VertIndexAndZ.Sort(FCompareIndexAndZ());

			// Search for duplicates, quickly!
			for (int32 i = 0; i < VertIndexAndZ.Num(); ++i)
			{
				const uint32 SrcVertIndex = VertIndexAndZ[i].Index;
				const float Z = VertIndexAndZ[i].Z;
				FSoftSkinVertex& SrcVert = CurSection.SoftVertices[SrcVertIndex];

				// only need to search forward, since we add pairs both ways
				for (int32 j = i + 1; j < VertIndexAndZ.Num(); ++j)
				{
					if (FMath::Abs(VertIndexAndZ[j].Z - Z) > THRESH_POINTS_ARE_SAME)
						break; // can't be any more dups

					const uint32 IterVertIndex = VertIndexAndZ[j].Index;
					FSoftSkinVertex& IterVert = CurSection.SoftVertices[IterVertIndex];
					if (PointsEqual(SrcVert.Position, IterVert.Position))
					{
						// if so, we add to overlapping vert
						TArray<int32>& SrcValueArray = CurSection.OverlappingVertices.FindOrAdd(SrcVertIndex);
						SrcValueArray.Add(IterVertIndex);

						TArray<int32>& IterValueArray = CurSection.OverlappingVertices.FindOrAdd(IterVertIndex);
						IterValueArray.Add(SrcVertIndex);
					}
				}
			}
		}
	};

	SkeletalMeshBuildData BuildData(
		LODModel,
		RefSkeleton,
		Influences,
		Wedges,
		Faces,
		Points,
		PointToOriginalMap,
		BuildOptions,
		OutWarningMessages,
		OutWarningNames);

	FSkeletalMeshUtilityBuilder Builder;
	if (!Builder.PrepareSourceMesh(SkeletalMeshName, BuildData))
	{
		return false;
	}

	if (!Builder.GenerateSkeletalRenderMesh(BuildData))
	{
		return false;
	}

	// Build the skeletal model from chunks.
	Builder.BeginSlowTask();
	BuildSkeletalModelFromChunks(BuildData.LODModel, BuildData.RefSkeleton, BuildData.Chunks, BuildData.PointToOriginalMap, BuildOptions.bCacheOptimize);
	UpdateOverlappingVertices(BuildData.LODModel);
	Builder.EndSlowTask();

	// Only show these warnings if in the game thread.  When importing morph targets, this function can run in another thread and these warnings dont prevent the mesh from importing
	if (IsInGameThread())
	{
		bool bHasBadSections = false;
		for (int32 SectionIndex = 0; SectionIndex < BuildData.LODModel.Sections.Num(); SectionIndex++)
		{
			FSkelMeshSection& Section = BuildData.LODModel.Sections[SectionIndex];
			bHasBadSections |= (Section.NumTriangles == 0);

			// Log info about the section.
			UE_LOGF(LogSkeletalMesh, Log, "Section %u: Material=%u, %u triangles",
				SectionIndex,
				Section.MaterialIndex,
				Section.NumTriangles
				);
		}
		if (bHasBadSections)
		{
			FText BadSectionMessage(NSLOCTEXT("UnrealEd", "Error_SkeletalMeshHasBadSections", "Input mesh has a section with no triangles.  This mesh may not render properly."));
			if (BuildData.OutWarningMessages)
			{
				BuildData.OutWarningMessages->Add(BadSectionMessage);
				if (BuildData.OutWarningNames)
				{
					BuildData.OutWarningNames->Add(FFbxErrors::SkeletalMesh_SectionWithNoTriangle);
				}
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, BadSectionMessage);
			}
		}

		if (BuildData.bTooManyVerts)
		{
			FText TooManyVertsMessage(NSLOCTEXT("UnrealEd", "Error_SkeletalMeshTooManyVertices", "Input mesh has too many vertices.  The generated mesh will be corrupt!  Consider adding extra materials to split up the source mesh into smaller chunks."));

			if (BuildData.OutWarningMessages)
			{
				BuildData.OutWarningMessages->Add(TooManyVertsMessage);
				if (BuildData.OutWarningNames)
				{
					BuildData.OutWarningNames->Add(FFbxErrors::SkeletalMesh_TooManyVertices);
				}
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, TooManyVertsMessage);
			}
		}
	}

	return true;
#else
	if (OutWarningMessages)
	{
		OutWarningMessages->Add(FText::FromString(TEXT("Cannot call FMeshUtilities::BuildSkeletalMesh on a console!")));
	}
	else
	{
		UE_LOGF(LogSkeletalMesh, Fatal, "Cannot call FMeshUtilities::BuildSkeletalMesh on a console!");
	}
	return false;
#endif
}

//The fail safe is there to avoid zeros in the tangents. Even if the fail safe prevent zero NTBs, a warning should be generate by the caller to let the artist know something went wrong
//Using a fail safe can lead to hard edge where its suppose to be smooth, it can also have some impact on the shading (lighting for tangentZ and normal map for tangentX and Y)
//Normally because we use the triangle data the tangent space is in a good direction and should give proper result.
void TangentFailSafe(const FVector3f &TriangleTangentX, const FVector3f &TriangleTangentY, const FVector3f &TriangleTangentZ
	, FVector3f &TangentX, FVector3f &TangentY, FVector3f &TangentZ)
{
	bool bTangentXZero = TangentX.IsNearlyZero() || TangentX.ContainsNaN();
	bool bTangentYZero = TangentY.IsNearlyZero() || TangentY.ContainsNaN();
	bool bTangentZZero = TangentZ.IsNearlyZero() || TangentZ.ContainsNaN();

	if (!bTangentXZero && !bTangentYZero && !bTangentZZero)
	{
		//No need to fail safe if everything is different from zero
		return;
	}
	if (!bTangentZZero)
	{
		if (!bTangentXZero)
		{
			//Valid TangentZ and TangentX, we can recompute TangentY
			TangentY = FVector3f::CrossProduct(TangentZ, TangentX).GetSafeNormal();
		}
		else if (!bTangentYZero)
		{
			//Valid TangentZ and TangentY, we can recompute TangentX
			TangentX = FVector3f::CrossProduct(TangentY, TangentZ).GetSafeNormal();
		}
		else
		{
			//TangentX and Y are invalid, use the triangle data, can cause a hard edge
			TangentX = TriangleTangentX.GetSafeNormal();
			TangentY = TriangleTangentY.GetSafeNormal();
		}
	}
	else if (!bTangentXZero)
	{
		if (!bTangentYZero)
		{
			//Valid TangentX and TangentY, we can recompute TangentZ
			TangentZ = FVector3f::CrossProduct(TangentX, TangentY).GetSafeNormal();
		}
		else
		{
			//TangentY and Z are invalid, use the triangle data, can cause a hard edge
			TangentZ = TriangleTangentZ.GetSafeNormal();
			TangentY = TriangleTangentY.GetSafeNormal();
		}
	}
	else if (!bTangentYZero)
	{
		//TangentX and Z are invalid, use the triangle data, can cause a hard edge
		TangentX = TriangleTangentX.GetSafeNormal();
		TangentZ = TriangleTangentZ.GetSafeNormal();
	}
	else
	{
		//Everything is zero, use all triangle data, can cause a hard edge
		TangentX = TriangleTangentX.GetSafeNormal();
		TangentY = TriangleTangentY.GetSafeNormal();
		TangentZ = TriangleTangentZ.GetSafeNormal();
	}

	bool bParaXY = FVector3f::Parallel(TangentX, TangentY);
	bool bParaYZ = FVector3f::Parallel(TangentY, TangentZ);
	bool bParaZX = FVector3f::Parallel(TangentZ, TangentX);
	if (bParaXY || bParaYZ || bParaZX)
	{
		//In case XY are parallel, use the Z(normal) if valid and not parallel to both X and Y to find the missing component
		if (bParaXY && !bParaZX)
		{
			TangentY = FVector3f::CrossProduct(TangentZ, TangentX).GetSafeNormal();
		}
		else if (bParaXY && !bParaYZ)
		{
			TangentX = FVector3f::CrossProduct(TangentY, TangentZ).GetSafeNormal();
		}
		else
		{
			//Degenerated value put something valid
			TangentX = FVector3f(1.0f, 0.0f, 0.0f);
			TangentY = FVector3f(0.0f, 1.0f, 0.0f);
			TangentZ = FVector3f(0.0f, 0.0f, 1.0f);
		}
	}
	else
	{
		//Ortho normalize the result
		TangentY -= TangentX * (TangentX | TangentY);
		TangentY.Normalize();

		TangentX -= TangentZ * (TangentZ | TangentX);
		TangentY -= TangentZ * (TangentZ | TangentY);

		TangentX.Normalize();
		TangentY.Normalize();

		//If we still have some zero data (i.e. triangle data is degenerated)
		if (TangentZ.IsNearlyZero() || TangentZ.ContainsNaN()
			|| TangentX.IsNearlyZero() || TangentX.ContainsNaN()
			|| TangentY.IsNearlyZero() || TangentY.ContainsNaN())
		{
			//Since the triangle is degenerate this case can cause a hardedge, but will probably have no other impact since the triangle is degenerate (no visible surface)
			TangentX = FVector3f(1.0f, 0.0f, 0.0f);
			TangentY = FVector3f(0.0f, 1.0f, 0.0f);
			TangentZ = FVector3f(0.0f, 0.0f, 1.0f);
		}
	}
}

static bool NonOpaqueMaterialPredicate(UStaticMeshComponent* InMesh)
{
	TArray<UMaterialInterface*> OutMaterials;
	InMesh->GetUsedMaterials(OutMaterials);
	for (auto Material : OutMaterials)
	{
		if (Material == nullptr || !IsOpaqueBlendMode(*Material))
		{
			return true;
		}
	}

	return false;
}

void FMeshUtilities::RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, FRawMesh &OutRawMesh) const
{
	// Compute any missing tangents.
	if (bRecomputeNormals || bRecomputeTangents)
	{
		float ComparisonThreshold = InBuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
		FOverlappingCorners OverlappingCorners;
		FindOverlappingCorners(OverlappingCorners, OutRawMesh, ComparisonThreshold);

		RecomputeTangentsAndNormalsForRawMesh( bRecomputeTangents, bRecomputeNormals, InBuildSettings, OverlappingCorners, OutRawMesh );
	}
}

void FMeshUtilities::RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, const FOverlappingCorners& InOverlappingCorners, FRawMesh &OutRawMesh) const
{
	const int32 NumWedges = OutRawMesh.WedgeIndices.Num();

	// Dump normals and tangents if we are recomputing them.
	if (bRecomputeTangents)
	{
		OutRawMesh.WedgeTangentX.Empty(NumWedges);
		OutRawMesh.WedgeTangentX.AddZeroed(NumWedges);
		OutRawMesh.WedgeTangentY.Empty(NumWedges);
		OutRawMesh.WedgeTangentY.AddZeroed(NumWedges);
	}

	if (bRecomputeNormals)
	{
		OutRawMesh.WedgeTangentZ.Empty(NumWedges);
		OutRawMesh.WedgeTangentZ.AddZeroed(NumWedges);
	}

	// Compute any missing tangents.
	if (bRecomputeNormals || bRecomputeTangents)
	{
		// Static meshes always blend normals of overlapping corners.
		uint32 TangentOptions = ETangentOptions::BlendOverlappingNormals;
		if (InBuildSettings.bRemoveDegenerates)
		{
			// If removing degenerate triangles, ignore them when computing tangents.
			TangentOptions |= ETangentOptions::IgnoreDegenerateTriangles;
		}

		if (InBuildSettings.bUseMikkTSpace)
		{
			ComputeTangents_MikkTSpace(OutRawMesh, InOverlappingCorners, TangentOptions);
		}
		else
		{
			ComputeTangents(OutRawMesh, InOverlappingCorners, TangentOptions);
		}
	}

	// At this point the mesh will have valid tangents.
	check(OutRawMesh.WedgeTangentX.Num() == NumWedges);
	check(OutRawMesh.WedgeTangentY.Num() == NumWedges);
	check(OutRawMesh.WedgeTangentZ.Num() == NumWedges);
}

void FMeshUtilities::ExtractMeshDataForGeometryCache(FRawMesh& RawMesh, const FMeshBuildSettings& BuildSettings, TArray<FStaticMeshBuildVertex>& OutVertices, TArray<TArray<uint32> >& OutPerSectionIndices, int32 ImportVersion)
{
	int32 NumWedges = RawMesh.WedgeIndices.Num();

	// Figure out if we should recompute normals and tangents. By default generated LODs should not recompute normals
	bool bRecomputeNormals = (BuildSettings.bRecomputeNormals) || RawMesh.WedgeTangentZ.Num() == 0;
	bool bRecomputeTangents = (BuildSettings.bRecomputeTangents) || RawMesh.WedgeTangentX.Num() == 0 || RawMesh.WedgeTangentY.Num() == 0;

	// Dump normals and tangents if we are recomputing them.
	if (bRecomputeTangents)
	{
		RawMesh.WedgeTangentX.Empty(NumWedges);
		RawMesh.WedgeTangentX.AddZeroed(NumWedges);
		RawMesh.WedgeTangentY.Empty(NumWedges);
		RawMesh.WedgeTangentY.AddZeroed(NumWedges);
	}

	if (bRecomputeNormals)
	{
		RawMesh.WedgeTangentZ.Empty(NumWedges);
		RawMesh.WedgeTangentZ.AddZeroed(NumWedges);
	}

	// Compute any missing tangents.
	FOverlappingCorners OverlappingCorners;
	if (bRecomputeNormals || bRecomputeTangents)
	{
		float ComparisonThreshold = GetComparisonThreshold(BuildSettings);
		FindOverlappingCorners(OverlappingCorners, RawMesh, ComparisonThreshold);

		// Static meshes always blend normals of overlapping corners.
		uint32 TangentOptions = ETangentOptions::BlendOverlappingNormals;
		if (BuildSettings.bRemoveDegenerates)
		{
			// If removing degenerate triangles, ignore them when computing tangents.
			TangentOptions |= ETangentOptions::IgnoreDegenerateTriangles;
		}
		if (BuildSettings.bUseMikkTSpace)
		{
			ComputeTangents_MikkTSpace(RawMesh, OverlappingCorners, TangentOptions);
		}
		else
		{
			ComputeTangents(RawMesh, OverlappingCorners, TangentOptions);
		}
	}

	// At this point the mesh will have valid tangents.
	check(RawMesh.WedgeTangentX.Num() == NumWedges);
	check(RawMesh.WedgeTangentY.Num() == NumWedges);
	check(RawMesh.WedgeTangentZ.Num() == NumWedges);

	TArray<int32> OutWedgeMap;

	int32 MaxMaterialIndex = 1;
	for (int32 FaceIndex = 0; FaceIndex < RawMesh.FaceMaterialIndices.Num(); FaceIndex++)
	{
		MaxMaterialIndex = FMath::Max<int32>(RawMesh.FaceMaterialIndices[FaceIndex], MaxMaterialIndex);
	}

	TMap<uint32, uint32> MaterialToSectionMapping;
	for (int32 i = 0; i <= MaxMaterialIndex; ++i)
	{
		OutPerSectionIndices.Push(TArray<uint32>());
		MaterialToSectionMapping.Add(i, i);
	}

	BuildStaticMeshVertexAndIndexBuffers(OutVertices, OutPerSectionIndices, OutWedgeMap, RawMesh, OverlappingCorners, MaterialToSectionMapping, KINDA_SMALL_NUMBER, (FVector3f)BuildSettings.BuildScale3D, ImportVersion);

	if (RawMesh.WedgeIndices.Num() < 100000 * 3)
	{
		CacheOptimizeVertexAndIndexBuffer(OutVertices, OutPerSectionIndices, OutWedgeMap);
		check(OutWedgeMap.Num() == RawMesh.WedgeIndices.Num());
	}
}

/*------------------------------------------------------------------------------
Mesh merging
------------------------------------------------------------------------------*/

void FMeshUtilities::CalculateTextureCoordinateBoundsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, TArray<FBox2D>& OutBounds) const
{
	TArray<FSoftSkinVertex> Vertices;
	LODModel.GetVertices(Vertices);

	const uint32 SectionCount = (uint32)LODModel.NumNonClothingSections();

	check(OutBounds.Num() != 0);

	for (uint32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		const uint32 FirstIndex = Section.BaseIndex;
		const uint32 LastIndex = FirstIndex + Section.NumTriangles * 3;
		const int32 MaterialIndex = Section.MaterialIndex;

		if (OutBounds.Num() <= MaterialIndex)
		{
			OutBounds.SetNumZeroed(MaterialIndex + 1);
		}

		for (uint32 Index = FirstIndex; Index < LastIndex; ++Index)
		{
			uint32 VertexIndex = LODModel.IndexBuffer[Index];
			FSoftSkinVertex& Vertex = Vertices[VertexIndex];

			FVector2D TexCoord = FVector2D(Vertex.UVs[0]);
			OutBounds[MaterialIndex] += TexCoord;
		}
	}
}

bool FMeshUtilities::RemoveBonesFromMesh(USkeletalMesh* SkeletalMesh, int32 LODIndex, const TArray<FName>* BoneNamesToRemove) const
{
	IMeshBoneReductionModule& MeshBoneReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshBoneReductionModule>("MeshBoneReduction");
	IMeshBoneReduction * MeshBoneReductionInterface = MeshBoneReductionModule.GetMeshBoneReductionInterface();

	return MeshBoneReductionInterface->ReduceBoneCounts(SkeletalMesh, LODIndex, BoneNamesToRemove);
}

bool FMeshUtilities::GetBoneReductionData(const USkeletalMesh* SkeletalMesh, int32 LODIndex, TMap<uint16, uint16>& OutBonesToReplace) const
{
	IMeshBoneReductionModule& MeshBoneReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshBoneReductionModule>("MeshBoneReduction");
	IMeshBoneReduction* MeshBoneReductionInterface = MeshBoneReductionModule.GetMeshBoneReductionInterface();

	return MeshBoneReductionInterface ? MeshBoneReductionInterface->GetBoneReductionData(SkeletalMesh, LODIndex, OutBonesToReplace) : false;
}

class FMeshSimplifcationSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable( new FMeshSimplifcationSettingsCustomization );
	}

	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override
	{
		MeshReductionModuleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshSimplificationSettings, MeshReductionModuleName));

		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("General")); 

		IDetailPropertyRow& PropertyRow = Category.AddProperty(MeshReductionModuleProperty);

		FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget();
		WidgetRow.NameContent()
		[
			MeshReductionModuleProperty->CreatePropertyNameWidget()
		];

		WidgetRow.ValueContent()
		.MaxDesiredWidth(0)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FMeshSimplifcationSettingsCustomization::GenerateMeshSimplifierMenu)
			.ContentPadding(FMargin(2.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FMeshSimplifcationSettingsCustomization::GetCurrentMeshSimplifierName)
			]
		];
	}

private:
	FText GetCurrentMeshSimplifierName() const
	{
		if(MeshReductionModuleProperty->IsValidHandle())
		{
			FText Name;
			MeshReductionModuleProperty->GetValueAsDisplayText(Name);

			return Name;
		}
		else
		{
			return LOCTEXT("AutomaticMeshReductionPlugin", "Automatic");
		}
	}

	TSharedRef<SWidget> GenerateMeshSimplifierMenu() const
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TArray<FName> ModuleNames;
		FModuleManager::Get().FindModules(TEXT("*MeshReduction"), ModuleNames);
		
		if(ModuleNames.Num() > 0)
		{
			for(FName ModuleName : ModuleNames)
			{
				IMeshReductionModule& Module = FModuleManager::LoadModuleChecked<IMeshReductionModule>(ModuleName);

				IMeshReduction* StaticMeshReductionInterface = Module.GetStaticMeshReductionInterface();
				// Only include options that support static mesh reduction.
				if (StaticMeshReductionInterface)
				{
					FUIAction UIAction;
					UIAction.ExecuteAction.BindSP(const_cast<FMeshSimplifcationSettingsCustomization*>(this), &FMeshSimplifcationSettingsCustomization::OnMeshSimplificationModuleChosen, ModuleName);
					UIAction.GetActionCheckState.BindSP(const_cast<FMeshSimplifcationSettingsCustomization*>(this), &FMeshSimplifcationSettingsCustomization::IsMeshSimplificationModuleChosen, ModuleName);

					MenuBuilder.AddMenuEntry(FText::FromName(ModuleName), FText::GetEmpty(), FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);
				}
			}

			MenuBuilder.AddMenuSeparator();
		}
		

		FUIAction OpenMarketplaceAction;
		OpenMarketplaceAction.ExecuteAction.BindSP(const_cast<FMeshSimplifcationSettingsCustomization*>(this), &FMeshSimplifcationSettingsCustomization::OnFindReductionPluginsClicked);
		FSlateIcon Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenMarketplace.Menu");
		MenuBuilder.AddMenuEntry( LOCTEXT("FindMoreReductionPluginsLink", "Search the Marketplace"), LOCTEXT("FindMoreReductionPluginsLink_Tooltip", "Opens the Marketplace to find more mesh reduction plugins"), Icon, OpenMarketplaceAction);
		return MenuBuilder.MakeWidget();
			}

	void OnMeshSimplificationModuleChosen(FName ModuleName)
			{
		if(MeshReductionModuleProperty->IsValidHandle())
				{
			MeshReductionModuleProperty->SetValue(ModuleName);
				}
			}

	ECheckBoxState IsMeshSimplificationModuleChosen(FName ModuleName)
			{
		if(MeshReductionModuleProperty->IsValidHandle())
			{
			FName CurrentModuleName;
			MeshReductionModuleProperty->GetValue(CurrentModuleName);
			return CurrentModuleName == ModuleName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

		return ECheckBoxState::Unchecked;
			}

	void OnFindReductionPluginsClicked()
	{
		FString URL;
		FUnrealEdMisc::Get().GetURL(TEXT("MeshSimplificationPluginsURL"), URL);

		FUnrealEdMisc::Get().OpenMarketplace(URL);
		}
private:
	TSharedPtr<IPropertyHandle> MeshReductionModuleProperty;
};

class FSkeletalMeshSimplificationSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FSkeletalMeshSimplificationSettingsCustomization);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		SkeletalMeshReductionModuleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletalMeshSimplificationSettings, SkeletalMeshReductionModuleName));

		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("General"));

		IDetailPropertyRow& PropertyRow = Category.AddProperty(SkeletalMeshReductionModuleProperty);

		FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget();
		WidgetRow.NameContent()
			[
				SkeletalMeshReductionModuleProperty->CreatePropertyNameWidget()
			];

		WidgetRow.ValueContent()
			.MaxDesiredWidth(0)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FSkeletalMeshSimplificationSettingsCustomization::GenerateSkeletalMeshSimplifierMenu)
			.ContentPadding(FMargin(2.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FSkeletalMeshSimplificationSettingsCustomization::GetCurrentSkeletalMeshSimplifierName)
			]
			];
	}

private:
	FText GetCurrentSkeletalMeshSimplifierName() const
	{
		if (SkeletalMeshReductionModuleProperty->IsValidHandle())
		{
			FText Name;
			SkeletalMeshReductionModuleProperty->GetValueAsDisplayText(Name);

			return Name;
		}
		else
		{
			return LOCTEXT("AutomaticSkeletalMeshReductionPlugin", "Automatic");
		}
	}

	TSharedRef<SWidget> GenerateSkeletalMeshSimplifierMenu() const
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TArray<FName> ModuleNames;
		FModuleManager::Get().FindModules(TEXT("*MeshReduction"), ModuleNames);

		if (ModuleNames.Num() > 0)
		{
			for (FName ModuleName : ModuleNames)
			{
				IMeshReductionModule& Module = FModuleManager::LoadModuleChecked<IMeshReductionModule>(ModuleName);

				IMeshReduction* SkeletalMeshReductionInterface = Module.GetSkeletalMeshReductionInterface();
				// Only include options that support skeletal simplification.
				if (SkeletalMeshReductionInterface)
				{
					FUIAction UIAction;
					UIAction.ExecuteAction.BindSP(const_cast<FSkeletalMeshSimplificationSettingsCustomization*>(this), &FSkeletalMeshSimplificationSettingsCustomization::OnSkeletalMeshSimplificationModuleChosen, ModuleName);
					UIAction.GetActionCheckState.BindSP(const_cast<FSkeletalMeshSimplificationSettingsCustomization*>(this), &FSkeletalMeshSimplificationSettingsCustomization::IsSkeletalMeshSimplificationModuleChosen, ModuleName);

					MenuBuilder.AddMenuEntry(FText::FromName(ModuleName), FText::GetEmpty(), FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);
				}
			}

			MenuBuilder.AddMenuSeparator();
		}


		FUIAction OpenMarketplaceAction;
		OpenMarketplaceAction.ExecuteAction.BindSP(const_cast<FSkeletalMeshSimplificationSettingsCustomization*>(this), &FSkeletalMeshSimplificationSettingsCustomization::OnFindReductionPluginsClicked);
		FSlateIcon Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenMarketplace.Menu");
		MenuBuilder.AddMenuEntry(LOCTEXT("FindMoreReductionPluginsLink", "Search the Marketplace"), LOCTEXT("FindMoreReductionPluginsLink_Tooltip", "Opens the Marketplace to find more mesh reduction plugins"), Icon, OpenMarketplaceAction);
		return MenuBuilder.MakeWidget();
	}

	void OnSkeletalMeshSimplificationModuleChosen(FName ModuleName)
	{
		if (SkeletalMeshReductionModuleProperty->IsValidHandle())
		{
			SkeletalMeshReductionModuleProperty->SetValue(ModuleName);
		}
	}

	ECheckBoxState IsSkeletalMeshSimplificationModuleChosen(FName ModuleName)
	{
		if (SkeletalMeshReductionModuleProperty->IsValidHandle())
		{
			FName CurrentModuleName;
			SkeletalMeshReductionModuleProperty->GetValue(CurrentModuleName);
			return CurrentModuleName == ModuleName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void OnFindReductionPluginsClicked()
	{
		FString URL;
		FUnrealEdMisc::Get().GetURL(TEXT("MeshSimplificationPluginsURL"), URL);

		FUnrealEdMisc::Get().OpenMarketplace(URL);
	}
private:
	TSharedPtr<IPropertyHandle> SkeletalMeshReductionModuleProperty;
};

class FProxyLODMeshSimplificationSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FProxyLODMeshSimplificationSettingsCustomization);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		ProxyLODMeshReductionModuleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UProxyLODMeshSimplificationSettings, ProxyLODMeshReductionModuleName));

		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("General"));

		IDetailPropertyRow& PropertyRow = Category.AddProperty(ProxyLODMeshReductionModuleProperty);

		FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget();
		WidgetRow.NameContent()
			[
				ProxyLODMeshReductionModuleProperty->CreatePropertyNameWidget()
			];

		WidgetRow.ValueContent()
			.MaxDesiredWidth(0)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FProxyLODMeshSimplificationSettingsCustomization::GenerateProxyLODMeshSimplifierMenu)
			.ContentPadding(FMargin(2.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FProxyLODMeshSimplificationSettingsCustomization::GetCurrentProxyLODMeshSimplifierName)
			]
			];
	}

private:
	FText GetCurrentProxyLODMeshSimplifierName() const
	{
		if (ProxyLODMeshReductionModuleProperty->IsValidHandle())
		{
			FText Name;
			ProxyLODMeshReductionModuleProperty->GetValueAsDisplayText(Name);

			return Name;
		}
		else
		{
			return LOCTEXT("AutomaticProxyLODMeshReductionPlugin", "Automatic");
		}
	}

	TSharedRef<SWidget> GenerateProxyLODMeshSimplifierMenu() const
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TArray<FName> ModuleNames;
		FModuleManager::Get().FindModules(TEXT("*MeshReduction"), ModuleNames);

		if (ModuleNames.Num() > 0)
		{
			for (FName ModuleName : ModuleNames)
			{
				IMeshReductionModule& Module = FModuleManager::LoadModuleChecked<IMeshReductionModule>(ModuleName);
				 
				IMeshMerging* MeshMergingInterface = Module.GetMeshMergingInterface();
				// Only include options that support mesh mergine.
				if (MeshMergingInterface)
				{
					FUIAction UIAction;
					UIAction.ExecuteAction.BindSP(const_cast<FProxyLODMeshSimplificationSettingsCustomization*>(this), &FProxyLODMeshSimplificationSettingsCustomization::OnProxyLODMeshSimplificationModuleChosen, ModuleName);
					UIAction.GetActionCheckState.BindSP(const_cast<FProxyLODMeshSimplificationSettingsCustomization*>(this), &FProxyLODMeshSimplificationSettingsCustomization::IsProxyLODMeshSimplificationModuleChosen, ModuleName);

					MenuBuilder.AddMenuEntry(FText::FromName(ModuleName), FText::GetEmpty(), FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);
				}
			}

			MenuBuilder.AddMenuSeparator();
		}


		FUIAction OpenMarketplaceAction;
		OpenMarketplaceAction.ExecuteAction.BindSP(const_cast<FProxyLODMeshSimplificationSettingsCustomization*>(this), &FProxyLODMeshSimplificationSettingsCustomization::OnFindReductionPluginsClicked);
		FSlateIcon Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenMarketplace.Menu");
		MenuBuilder.AddMenuEntry(LOCTEXT("FindMoreReductionPluginsLink", "Search the Marketplace"), LOCTEXT("FindMoreReductionPluginsLink_Tooltip", "Opens the Marketplace to find more mesh reduction plugins"), Icon, OpenMarketplaceAction);
		return MenuBuilder.MakeWidget();
	}

	void OnProxyLODMeshSimplificationModuleChosen(FName ModuleName)
	{
		if (ProxyLODMeshReductionModuleProperty->IsValidHandle())
		{
			ProxyLODMeshReductionModuleProperty->SetValue(ModuleName);
		}
	}

	ECheckBoxState IsProxyLODMeshSimplificationModuleChosen(FName ModuleName)
	{
		if (ProxyLODMeshReductionModuleProperty->IsValidHandle())
		{
			FName CurrentModuleName;
			ProxyLODMeshReductionModuleProperty->GetValue(CurrentModuleName);
			return CurrentModuleName == ModuleName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void OnFindReductionPluginsClicked()
	{
		FString URL;
		FUnrealEdMisc::Get().GetURL(TEXT("MeshSimplificationPluginsURL"), URL);

		FUnrealEdMisc::Get().OpenMarketplace(URL);
	}
private:
	TSharedPtr<IPropertyHandle> ProxyLODMeshReductionModuleProperty;
};

/*------------------------------------------------------------------------------
Module initialization / teardown.
------------------------------------------------------------------------------*/

void FMeshUtilities::StartupModule()
{
	FModuleManager::Get().LoadModule("MaterialBaking");
	FModuleManager::Get().LoadModule(TEXT("MeshMergeUtilities"));
	FModuleManager::Get().LoadModule(TEXT("MeshBoneReduction"));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout("MeshSimplificationSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshSimplifcationSettingsCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout("SkeletalMeshSimplificationSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalMeshSimplificationSettingsCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout("ProxyLODMeshSimplificationSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FProxyLODMeshSimplificationSettingsCustomization::MakeInstance));

	static const TConsoleVariableData<int32>* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.TriangleOrderOptimization"));

	bDisableTriangleOrderOptimization = (CVar->GetValueOnGameThread() == 2);

	bUsingNvTriStrip = !bDisableTriangleOrderOptimization && (CVar->GetValueOnGameThread() == 0);

	IMeshReductionManagerModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	IMeshReduction* StaticMeshReduction = Module.GetStaticMeshReductionInterface();


	// Construct and cache the version string for the mesh utilities module.
	VersionString = FString::Printf(
		TEXT("%s%s%s"),
		MESH_UTILITIES_VER,
		StaticMeshReduction ? *StaticMeshReduction->GetVersionString() : TEXT(""),
		bUsingNvTriStrip ? TEXT("_NvTriStrip") : TEXT("")
		);

	// hook up level editor extension for skeletal mesh conversion
	ModuleLoadedDelegateHandle = FModuleManager::Get().OnModulesChanged().AddLambda([this](FName InModuleName, EModuleChangeReason InChangeReason)
	{
		if (InChangeReason == EModuleChangeReason::ModuleLoaded)
		{
			if (InModuleName == "LevelEditor")
			{
				AddLevelViewportMenuExtender();
			}
		}
	});

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FMeshUtilities::RegisterMenus));
}

void FMeshUtilities::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	static const FName PropertyEditorModuleName("PropertyEditor");
	if(FModuleManager::Get().IsModuleLoaded(PropertyEditorModuleName))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

		PropertyEditorModule.UnregisterCustomClassLayout("MeshSimplificationSettings");
		PropertyEditorModule.UnregisterCustomClassLayout("SkeletalMeshSimplificationSettings");
		PropertyEditorModule.UnregisterCustomClassLayout("ProxyLODMeshSimplificationSettings");
	}

	RemoveLevelViewportMenuExtender();
	FModuleManager::Get().OnModulesChanged().Remove(ModuleLoadedDelegateHandle);
	VersionString.Empty();
}

void FMeshUtilities::RegisterMenus()
{
	AddMakeStaticMeshEntryToToolMenu("AssetEditor.AnimationEditor.ToolBar");
	AddMakeStaticMeshEntryToToolMenu("AssetEditor.AnimationBlueprintEditor.MainMenu.Tools");
	AddMakeStaticMeshEntryToToolMenu("AssetEditor.SkeletalMeshEditor.ToolBar");
	AddMakeStaticMeshEntryToToolMenu("AssetEditor.SkeletonEditor.ToolBar");
}

bool FMeshUtilities::GenerateUniqueUVsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const
{
	// Get easy to use SkeletalMesh data
	TArray<FSoftSkinVertex> Vertices;
	LODModel.GetVertices(Vertices);

	int32 NumCorners = LODModel.IndexBuffer.Num();

	// Generate FRawMesh from FSkeletalMeshLODModel
	FRawMesh TempMesh;
	TempMesh.WedgeIndices.AddUninitialized(NumCorners);
	TempMesh.WedgeTexCoords[0].AddUninitialized(NumCorners);
	TempMesh.VertexPositions.AddUninitialized(NumCorners);

	// Prepare vertex to wedge map
	// PrevCorner[i] points to previous corner which shares the same wedge
	TArray<int32> LastWedgeCorner;
	LastWedgeCorner.AddUninitialized(Vertices.Num());
	TArray<int32> PrevCorner;
	PrevCorner.AddUninitialized(NumCorners);
	for (int32 Index = 0; Index < Vertices.Num(); Index++)
	{
		LastWedgeCorner[Index] = -1;
	}

	for (int32 Index = 0; Index < NumCorners; Index++)
	{
		// Copy static vertex data
		int32 VertexIndex = LODModel.IndexBuffer[Index];
		FSoftSkinVertex& Vertex = Vertices[VertexIndex];
		TempMesh.WedgeIndices[Index] = Index; // rudimental data, not really used by FLayoutUV - but array size matters
		TempMesh.WedgeTexCoords[0][Index] = Vertex.UVs[0];
		TempMesh.VertexPositions[Index] = Vertex.Position;
		// Link all corners belonging to a single wedge into list
		int32 PrevCornerIndex = LastWedgeCorner[VertexIndex];
		LastWedgeCorner[VertexIndex] = Index;
		PrevCorner[Index] = PrevCornerIndex;
	}

	// Build overlapping corners map
	FOverlappingCorners OverlappingCorners;
	OverlappingCorners.Init(NumCorners);
	for (int32 Index = 0; Index < NumCorners; Index++)
	{
		int VertexIndex = LODModel.IndexBuffer[Index];
		for (int32 CornerIndex = LastWedgeCorner[VertexIndex]; CornerIndex >= 0; CornerIndex = PrevCorner[CornerIndex])
		{
			if (CornerIndex != Index)
			{
				OverlappingCorners.Add(Index, CornerIndex);
			}
		}
	}
	OverlappingCorners.FinishAdding();

	// Generate new UVs
	FLayoutUVRawMeshView TempMeshView(TempMesh, 0, 1);
	FLayoutUV Packer(TempMeshView);
	Packer.FindCharts(OverlappingCorners);

	bool bPackSuccess = Packer.FindBestPacking(FMath::Clamp(TextureResolution / 4, 32, 512));
	if (bPackSuccess)
	{
		Packer.CommitPackedUVs();
		// Save generated UVs
		OutTexCoords = TempMesh.WedgeTexCoords[1];
	}
	return bPackSuccess;
}

void FMeshUtilities::CalculateTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY, TArray<FVector3f>& OutNormals) const
{
	const float ComparisonThreshold = (InTangentOptions & ETangentOptions::IgnoreDegenerateTriangles) ? THRESH_POINTS_ARE_SAME : 0.0f;

	FOverlappingCorners OverlappingCorners;
	FindOverlappingCorners(OverlappingCorners, InVertices, InIndices, ComparisonThreshold);

	if (InTangentOptions & ETangentOptions::UseMikkTSpace)
	{
		ComputeTangents_MikkTSpace(InVertices, InIndices, InUVs, InSmoothingGroupIndices, OverlappingCorners, OutTangentX, OutTangentY, OutNormals, InTangentOptions);
	}
	else
	{
		ComputeTangents(InVertices, InIndices, InUVs, InSmoothingGroupIndices, OverlappingCorners, OutTangentX, OutTangentY, OutNormals, InTangentOptions);
	}
}

void FMeshUtilities::CalculateMikkTSpaceTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<FVector3f>& InNormals, bool bIgnoreDegenerateTriangles, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY) const
{
	ComputeTangents_MikkTSpace(InVertices, InIndices, InUVs, InNormals, bIgnoreDegenerateTriangles, OutTangentX, OutTangentY);
}

void FMeshUtilities::CalculateNormals(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutNormals) const
{
	const float ComparisonThreshold = (InTangentOptions & ETangentOptions::IgnoreDegenerateTriangles ) ? THRESH_POINTS_ARE_SAME : 0.0f;

	FOverlappingCorners OverlappingCorners;
	FindOverlappingCorners(OverlappingCorners, InVertices, InIndices, ComparisonThreshold);

	ComputeNormals(InVertices, InIndices, InUVs, InSmoothingGroupIndices, OverlappingCorners, OutNormals, InTangentOptions);
}

void FMeshUtilities::CalculateOverlappingCorners(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, bool bIgnoreDegenerateTriangles, FOverlappingCorners& OutOverlappingCorners) const
{
	const float ComparisonThreshold = bIgnoreDegenerateTriangles ? THRESH_POINTS_ARE_SAME : 0.f;
	FindOverlappingCorners(OutOverlappingCorners, InVertices, InIndices, ComparisonThreshold);
}


struct FHashableSkinWeight
{
	FHashableSkinWeight() = default;
	FHashableSkinWeight(const FHashableSkinWeight&) = default;
	FHashableSkinWeight(FHashableSkinWeight&&) = default;
	FHashableSkinWeight(const FRawSkinWeight& InRawWeight, const int32 InNumInfluences) :
		NumInfluences(InNumInfluences),
		InfluenceBones(InRawWeight.InfluenceBones),
		InfluenceWeights(InRawWeight.InfluenceWeights)
	{
	}
	
	const int32 NumInfluences = 0;
	const FBoneIndexType *InfluenceBones = nullptr;
	const uint16 *InfluenceWeights = nullptr;
};

struct FHashableSkinWeightKeyFuncs : BaseKeyFuncs<TPair<FHashableSkinWeight, int32>, FHashableSkinWeight>
{
	static const FHashableSkinWeight& GetSetKey(const TPair<FHashableSkinWeight, int32>& Element)
	{
		return Element.Key;
	}

	static bool Matches(const FHashableSkinWeight& A, const FHashableSkinWeight& B)
	{
		for (int32 Index = 0; Index < A.NumInfluences; Index++)
		{
			if (A.InfluenceBones[Index] != B.InfluenceBones[Index] ||
				A.InfluenceWeights[Index] != B.InfluenceWeights[Index])
			{
				return false;
			}
		}
		return true;
	}
	
	static uint32 GetKeyHash(const FHashableSkinWeight& Key)
	{
		uint32 Hash = GetTypeHash(Key.NumInfluences);
		for (int32 Index = 0; Index < Key.NumInfluences; Index++)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(Key.InfluenceBones[Index]));
			Hash = HashCombineFast(Hash, GetTypeHash(Key.InfluenceWeights[Index]));
		}
		return Hash;
	}
};


void FMeshUtilities::GenerateRuntimeSkinWeightData(
	const FSkeletalMeshLODModel* ImportedModel,
	const TArray<FRawSkinWeight>& InRawSkinWeights,
	bool bInUseHighPrecisionWeights,
	FRuntimeSkinWeightProfileData& InOutSkinWeightOverrideData,
	const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices) const
{
	const FSkeletalMeshLODModel& TargetLODModel = *ImportedModel;

	// Make sure the number of verts of the LOD matches the provided number of skin weights
	if (InRawSkinWeights.Num() == TargetLODModel.NumVertices)
	{
		// Retrieve all vertices for this LOD
		TArray<FSoftSkinVertex> TargetVertices;
		TargetLODModel.GetVertices(TargetVertices, UnifiedBoneIndices);

		// Determine how many influences each skinweight can contain
		const int32 NumInfluences = TargetLODModel.GetMaxBoneInfluences();
		InOutSkinWeightOverrideData.NumWeightsPerVertex = NumInfluences;

		const bool b16BitBoneIndices = (UnifiedBoneIndices && UnifiedBoneIndices->GetNumUnifiedBones() > 256) || TargetLODModel.DoSectionsUse16BitBoneIndex();
		InOutSkinWeightOverrideData.b16BitBoneIndices = b16BitBoneIndices;

		TMap<FHashableSkinWeight, int32, FDefaultSetAllocator, FHashableSkinWeightKeyFuncs> UniqueWeights;
		int32 OverrideCount = 0;
		
		for (int32 VertexIndex = 0; VertexIndex < TargetVertices.Num(); ++VertexIndex)
		{
			// Take each original skin weight from the LOD and compare it with supplied alternative weight data
			const FRawSkinWeight& SourceSkinWeight = InRawSkinWeights[VertexIndex];
			const FSoftSkinVertex& TargetVertex = TargetVertices[VertexIndex];

			bool bIsDifferent = false;
			// Weights are generally sorted from high to low, but they are not guaranteed to be sorted by bone index.
			// Since we want to know if two list of weights are the same, we have to sort them again by both index and weight
			{
				struct FInfluence
				{
					uint16 BoneIndex = 0;
					uint16 Weight = 0;

					bool operator < (const FInfluence& Other) const
					{
						if (Weight == Other.Weight)
						{
							return BoneIndex < Other.BoneIndex;
						}
						
						return Weight > Other.Weight;
					}

					bool operator == (const FInfluence& Other) const = default;
				};


				TArray<FInfluence, TInlineAllocator<MAX_TOTAL_INFLUENCES>> SourceWeightArray;
				TArray<FInfluence, TInlineAllocator<MAX_TOTAL_INFLUENCES>> TargetWeightArray;
			
				for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
				{
					SourceWeightArray.Add({SourceSkinWeight.InfluenceBones[InfluenceIndex], SourceSkinWeight.InfluenceWeights[InfluenceIndex]});
					TargetWeightArray.Add({TargetVertex.InfluenceBones[InfluenceIndex], TargetVertex.InfluenceWeights[InfluenceIndex]});
				}

				Algo::Sort(SourceWeightArray);
				Algo::Sort(TargetWeightArray);
			
			
				for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
				{
					if (SourceWeightArray[InfluenceIndex] != TargetWeightArray[InfluenceIndex])
					{
						bIsDifferent = true;
						break;
					}
				}
			}

			if (bIsDifferent)
			{
				const int32* OverrideIndexPtr = UniqueWeights.Find({SourceSkinWeight, NumInfluences});
				
				// If one hasn't been added yet, create a new one
				int32 OverrideIndex;
				if (OverrideIndexPtr == nullptr)
				{
					// When writing out 8-bit weights, we want to collect the resulting deviation from fully normalized
					// weight sum. Once all values are copied, we alter the first, and most prominent, weight, by the 
					// deviation value so that we end up with completely normalized 8-bit weights.
					const int32 FirstWeightOffset = InOutSkinWeightOverrideData.BoneWeights.Num();
					int32 Accumulated8BitDeviation = 255;

					// Write out non-zero weighted influences only
					for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
					{
						const FBoneIndexType Index = SourceSkinWeight.InfluenceBones[InfluenceIndex];
						const uint16 Weight = SourceSkinWeight.InfluenceWeights[InfluenceIndex];
						TArray<uint8> &BoneIDs = InOutSkinWeightOverrideData.BoneIDs;
						TArray<uint8> &BoneWeights = InOutSkinWeightOverrideData.BoneWeights;

						if (b16BitBoneIndices)
						{
							BoneIDs.AddZeroed(2);
							*reinterpret_cast<FBoneIndexType*>(&BoneIDs[BoneIDs.Num() - 2]) = Index;
						}
						else
						{
							InOutSkinWeightOverrideData.BoneIDs.Add(static_cast<uint8>(Index));
						}

						if (bInUseHighPrecisionWeights)
						{
							BoneWeights.AddZeroed(2);
							*reinterpret_cast<uint16*>(&BoneWeights[BoneWeights.Num() - 2]) = Weight;
						}
						else
						{
							BoneWeights.Add(static_cast<uint8>(Weight >> 8));
							Accumulated8BitDeviation -= BoneWeights.Last();
						}
					}

					OverrideIndex = OverrideCount;
					UniqueWeights.Add({SourceSkinWeight, NumInfluences}, OverrideCount);
					OverrideCount++;

					if (!bInUseHighPrecisionWeights)
					{
						InOutSkinWeightOverrideData.BoneWeights[FirstWeightOffset] += Accumulated8BitDeviation;
					}
				}
				else
				{
					OverrideIndex = *OverrideIndexPtr;
				}

				InOutSkinWeightOverrideData.VertexIndexToInfluenceOffset.Add(VertexIndex, OverrideIndex);
			}
		}
	}
}

void FMeshUtilities::CreateImportDataFromLODModel(USkeletalMesh* SkeletalMesh, bool bInResetReductionAsNeeded) const
{
	check(IsInGameThread());

	for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); LodIndex++)
	{
		FSkeletalMeshLODInfo* ThisLODInfo = SkeletalMesh->GetLODInfo(LodIndex);

		check(ThisLODInfo);
		if (SkeletalMesh->HasMeshDescription(LodIndex))
		{
			//No need to create import data if we already have some
			continue;
		}

		// If the mesh was not pulled out of the reduction data, we need to reset the LOD settings
		// so that the mesh doesn't get reduced again if it gets regenerated.
		const bool bReductionActive = SkeletalMesh->IsReductionActive(LodIndex);
		const bool bInlineReduction = (ThisLODInfo->ReductionSettings.BaseLOD == LodIndex);
		if (bReductionActive && !bInlineReduction)
		{
			//Generated LOD (not inline) do not need imported data
			continue;
		}

		//Find the reduction data to restore the imported LODModel by using the deprecated OriginalReductionSourceMeshData_DEPRECATED data
		if (SkeletalMesh->GetImportedModel()->OriginalReductionSourceMeshData_DEPRECATED.IsValidIndex(LodIndex))
		{
			FSkeletalMeshLODModel TempLODModel;
			TMap<FString, TArray<FMorphTargetDelta>> LODMorphTargetData;
			
			//Old inline reduced assets do not have the original imported skin weight data, it must be re-import if we want to convert it to import data
			SkeletalMesh->GetImportedModel()->OriginalReductionSourceMeshData_DEPRECATED[LodIndex]->LoadReductionData(TempLODModel, LODMorphTargetData, SkeletalMesh);

			if (TempLODModel.NumVertices > 0)
			{
				FSkeletalMeshLODModel& ToUpdateLODModel = SkeletalMesh->GetImportedModel()->LODModels[LodIndex];

				// Unbind any existing clothing assets before loading the reduction data
				TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
				FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ClothingBindings, LodIndex);

				//Swap the LODModel pointer to the one we store when reducing, so we add the true import data
				FSkeletalMeshLODModel::CopyStructure(&ToUpdateLODModel, &TempLODModel);

				//Rebind the cloth asset on the LODModel restore by the reduction data
				FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ClothingBindings, LodIndex);

				//Since we load the reduction LODModel we have to validate the material indexes.
				SkeletalMesh->ValidateAllLodMaterialIndexes();

				TArray<UMorphTarget*> OriginalMorphTargets;
				OriginalMorphTargets.Reserve(LODMorphTargetData.Num());
				for (const TPair<FString, TArray<FMorphTargetDelta>>& MorphTargetChannel : LODMorphTargetData)
				{
					UMorphTarget* ExistingMorphTarget = nullptr;
					for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
					{
						if (MorphTarget->GetName() == MorphTargetChannel.Key)
						{
							ExistingMorphTarget = MorphTarget;
						}
					}
					if (!ExistingMorphTarget)
					{
						FName MorphTargetName = *(MorphTargetChannel.Key);
						ExistingMorphTarget = NewObject<UMorphTarget>(SkeletalMesh, MorphTargetName);
						check(ExistingMorphTarget);
					}

					const TArray<FMorphTargetDelta>& MorphTargetDeltas = MorphTargetChannel.Value;
					ExistingMorphTarget->PopulateDeltas(MorphTargetDeltas, LodIndex, ToUpdateLODModel.Sections, ThisLODInfo->BuildSettings.bRecomputeNormals, false, ThisLODInfo->BuildSettings.MorphThresholdPosition);
				}
			}
			
			// The model got pulled out of the reduction storage, don't reset the LOD reduction settings,
			// since we may want to regenerate the mesh to those settings.
			bInResetReductionAsNeeded = false;
		}

		FSkeletalMeshLODModel* LODModel = &(SkeletalMesh->GetImportedModel()->LODModels[LodIndex]);

		//Make sure the inline cache fit the LODModel
		if (SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas.IsValidIndex(LodIndex))
		{
			SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas[LodIndex].SetCacheGeometryInfo(*LODModel);
		}

		FSkeletalMeshImportData ImportData;
		ImportData.bHasNormals = true;
		ImportData.bHasTangents = true;
		ImportData.bHasVertexColors = SkeletalMesh->GetHasVertexColors();

		TArray<FSkeletalMaterial>& SKMaterials = SkeletalMesh->GetMaterials();
		ImportData.Materials.Reserve(SkeletalMesh->GetMaterials().Num());
		TArray<FName> UniqueMaterialNames;
		for (int32 MaterialIndex = 0; MaterialIndex < SKMaterials.Num(); ++MaterialIndex)
		{
			SkeletalMeshImportData::FMaterial& Material = ImportData.Materials.AddZeroed_GetRef();
			Material.Material = SKMaterials[MaterialIndex].MaterialInterface;
			//Make sure skeletal mesh material has the slot name set properly
			if (SKMaterials[MaterialIndex].ImportedMaterialSlotName == NAME_None)
			{
				if (SKMaterials[MaterialIndex].MaterialInterface)
				{
					SKMaterials[MaterialIndex].ImportedMaterialSlotName = SKMaterials[MaterialIndex].MaterialInterface->GetFName();
				}
				else if (SKMaterials[MaterialIndex].MaterialSlotName != NAME_None)
				{
					SKMaterials[MaterialIndex].ImportedMaterialSlotName = SKMaterials[MaterialIndex].MaterialSlotName;
				}
				else
				{
					//We cannot find any valid name
					UE_ASSET_LOG(LogSkeletalMesh, Display, SkeletalMesh, TEXT("FMeshUtilities::CreateImportDataFromLODModel: Cannot assign valid slotname to material slot"));
					SKMaterials[MaterialIndex].ImportedMaterialSlotName = NAME_None;
				}
				//each material slot must have a unique imported material name
				while (UniqueMaterialNames.Contains(SKMaterials[MaterialIndex].ImportedMaterialSlotName))
				{
					const int32 NameUniqueNumber = SKMaterials[MaterialIndex].ImportedMaterialSlotName.GetNumber() + 1;
					SKMaterials[MaterialIndex].ImportedMaterialSlotName.SetNumber(NameUniqueNumber);
				}
			}
			if (SKMaterials[MaterialIndex].MaterialSlotName == NAME_None)
			{
				SKMaterials[MaterialIndex].MaterialSlotName = SKMaterials[MaterialIndex].ImportedMaterialSlotName;
			}
			UniqueMaterialNames.Add(SKMaterials[MaterialIndex].ImportedMaterialSlotName);

			Material.MaterialImportName = SKMaterials[MaterialIndex].ImportedMaterialSlotName.ToString();
		}

		ImportData.NumTexCoords = FMath::Min(LODModel->NumTexCoords, static_cast<uint32>(MAX_TEXCOORDS));

		const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();
		for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetRawBoneNum(); ++BoneIndex)
		{
			SkeletalMeshImportData::FBone& RefBone = ImportData.RefBonesBinary.AddZeroed_GetRef();
			const FMeshBoneInfo& MeshBoneInfo = ReferenceSkeleton.GetRawRefBoneInfo()[BoneIndex];
			const FTransform& MeshBonePose = ReferenceSkeleton.GetRawRefBonePose()[BoneIndex];
			RefBone.Name = MeshBoneInfo.ExportName;
			RefBone.BonePos.Transform = FTransform3f(MeshBonePose);
			RefBone.ParentIndex = MeshBoneInfo.ParentIndex;
			RefBone.NumChildren = 0;
			if (ImportData.RefBonesBinary.IsValidIndex(RefBone.ParentIndex))
			{
				SkeletalMeshImportData::FBone& ParentBone = ImportData.RefBonesBinary[RefBone.ParentIndex];
				ParentBone.NumChildren++;
			}
		}

		TArray<FSoftSkinVertex> Vertices;
		LODModel->GetVertices(Vertices);
		const int32 VertexCount = Vertices.Num();
		const int32 TriangleCount = LODModel->IndexBuffer.Num() / 3;
		ImportData.Points.Reserve(VertexCount);
		ImportData.PointToRawMap.Reserve(VertexCount);
		ImportData.Wedges.Reserve(VertexCount);
		ImportData.Faces.Reserve(TriangleCount);

		for (int32 SectionIndex = 0; SectionIndex < LODModel->Sections.Num(); ++SectionIndex)
		{
			const FSkelMeshSection& Section = LODModel->Sections[SectionIndex];
			int32 VerticeIndexStart = Section.BaseVertexIndex;
			int32 VerticeIndexEnd = VerticeIndexStart + Section.GetNumVertices();
			int32 MaterialIndex = Section.MaterialIndex;
			ImportData.MaxMaterialIndex = FMath::Max(ImportData.MaxMaterialIndex, static_cast<uint32>(MaterialIndex));
			for (int32 VerticeIndex = VerticeIndexStart; VerticeIndex < VerticeIndexEnd; ++VerticeIndex)
			{
				const FSoftSkinVertex& Vertex = Vertices[VerticeIndex];
				int32 PointIndex = ImportData.Points.Add(Vertex.Position);
				ImportData.PointToRawMap.Add(PointIndex);

				for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				{
					// Have we reached the end of valid weights?
					if (Vertex.InfluenceWeights[InfluenceIndex] == 0)
					{
						break;
					}
					SkeletalMeshImportData::FRawBoneInfluence& Influence = ImportData.Influences.AddZeroed_GetRef();
					Influence.VertexIndex = PointIndex;
					Influence.BoneIndex = Section.BoneMap[Vertex.InfluenceBones[InfluenceIndex]];
					Influence.Weight = static_cast<float>(Vertex.InfluenceWeights[InfluenceIndex]) / 65535.0f;
				}
			}
		}

		for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			SkeletalMeshImportData::FTriangle& Face = ImportData.Faces.AddZeroed_GetRef();
			int32 IndexBufferIndex = TriangleIndex * 3;
			int32 SectionIndex;
			int32 VerticeIndex[3];
			int32 SectionVertexIndex[3];
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const int32 WedgeIndex = ImportData.Wedges.Num();
				VerticeIndex[Corner] = LODModel->IndexBuffer[IndexBufferIndex + Corner];
				int32 NextEdgeCornerIndex = (Corner + 1) % 3;
				LODModel->GetSectionFromVertexIndex(VerticeIndex[Corner], SectionIndex, SectionVertexIndex[Corner]);
				const FSoftSkinVertex& Vertex = Vertices[VerticeIndex[Corner]];
				SkeletalMeshImportData::FVertex& Wedge = ImportData.Wedges.AddZeroed_GetRef();
				Wedge.Color = Vertex.Color;
				for (int32 UVIndex = 0; UVIndex < static_cast<int32>(ImportData.NumTexCoords); ++UVIndex)
				{
					Wedge.UVs[UVIndex] = Vertex.UVs[UVIndex];
				}
				Wedge.VertexIndex = VerticeIndex[Corner];
				Wedge.MatIndex = LODModel->Sections[SectionIndex].MaterialIndex;

				Face.WedgeIndex[Corner] = WedgeIndex;
				Face.TangentX[Corner] = Vertex.TangentX;
				Face.TangentY[Corner] = Vertex.TangentY;
				Face.TangentZ[Corner] = Vertex.TangentZ;
			}
			Face.MatIndex = LODModel->Sections[SectionIndex].MaterialIndex;
			//Faceted by default
			Face.SmoothingGroups = 0x0;
		}

		//Recreate smooth group
		ImportData.ComputeSmoothGroupFromNormals();

		const int32 SrcMorphTargetNum = SkeletalMesh->GetMorphTargets().Num();
		ImportData.MorphTargetNames.Reserve(SrcMorphTargetNum);
		ImportData.MorphTargetModifiedPoints.Reserve(SrcMorphTargetNum);
		ImportData.MorphTargets.Reserve(SrcMorphTargetNum);
		for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
		{
			if (!MorphTarget->HasDataForLOD(LodIndex))
			{
				continue;
			}
			ImportData.MorphTargetNames.Add(MorphTarget->GetName());
			const FMorphTargetLODModel& BaseMorphModel = MorphTarget->GetMorphLODModels()[LodIndex];
			FSkeletalMeshImportData& MorphImportShape = ImportData.MorphTargets.AddDefaulted_GetRef();
			TSet<uint32>& ModifiedPoints = ImportData.MorphTargetModifiedPoints.AddDefaulted_GetRef();
			//Iterate each original morph target source index to fill the NewMorphTargetDeltas array with the TargetMatchData.
			for (const FMorphTargetDelta& MorphTargetDelta : BaseMorphModel.Vertices)
			{
				ModifiedPoints.Add(MorphTargetDelta.SourceIdx);
				MorphImportShape.Points.Add(MorphTargetDelta.PositionDelta + Vertices[MorphTargetDelta.SourceIdx].Position);
			}
		}

		if (LODModel->SkinWeightProfiles.Num() > 0)
		{
			FSkeletalMeshImportData ReferenceCopy;
			ReferenceCopy = ImportData;
			ReferenceCopy.KeepAlternateSkinningBuildDataOnly();
			ReferenceCopy.Influences.Empty();
			for (TPair<FName, FImportedSkinWeightProfileData>& SkinWeightProfile : LODModel->SkinWeightProfiles)
			{
				FString ProfileName = SkinWeightProfile.Key.ToString();
				FImportedSkinWeightProfileData& ImportedSkinWeightProfileData = SkinWeightProfile.Value;
				if (!ensure(VertexCount == ImportedSkinWeightProfileData.SkinWeights.Num()))
				{
					UE_ASSET_LOG(LogSkeletalMesh, Log, SkeletalMesh, TEXT("Cannot convert alternate skin weight profile [%s]"), *ProfileName);
					continue;
				}
				ImportData.AlternateInfluenceProfileNames.Add(ProfileName);
				FSkeletalMeshImportData& AlternateInfluence = ImportData.AlternateInfluences.AddDefaulted_GetRef();
				AlternateInfluence = ReferenceCopy;
				AlternateInfluence.Influences.Empty(VertexCount* MAX_TOTAL_INFLUENCES);
				ImportedSkinWeightProfileData.SourceModelInfluences.Empty(VertexCount * MAX_TOTAL_INFLUENCES);
				for (int32 VerticeIndex = 0; VerticeIndex < VertexCount; ++VerticeIndex)
				{
					int32 OutSectionIndex = INDEX_NONE;
					int32 OutSectionVertexIndex = INDEX_NONE;
					LODModel->GetSectionFromVertexIndex(VerticeIndex, OutSectionIndex, OutSectionVertexIndex);
					const FSkelMeshSection& Section = LODModel->Sections[OutSectionIndex];
					const FRawSkinWeight& SkinWeight = ImportedSkinWeightProfileData.SkinWeights[VerticeIndex];
					for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
					{
						const uint16 RawWeight = SkinWeight.InfluenceWeights[InfluenceIndex]; 
						if (RawWeight == 0)
						{
							break;
						}
						const float Weight = static_cast<float>(RawWeight) / 65535.0f;
						SkeletalMeshImportData::FRawBoneInfluence& Influence = AlternateInfluence.Influences.AddZeroed_GetRef();
						Influence.VertexIndex = VerticeIndex;
						Influence.BoneIndex = Section.BoneMap[SkinWeight.InfluenceBones[InfluenceIndex]];
						Influence.Weight = Weight;
						SkeletalMeshImportData::FVertInfluence& SourceModelInfluence = ImportedSkinWeightProfileData.SourceModelInfluences.AddZeroed_GetRef();
						SourceModelInfluence.VertIndex = VerticeIndex;
						SourceModelInfluence.BoneIndex = SkinWeight.InfluenceBones[InfluenceIndex];
						SourceModelInfluence.Weight = Weight;
					}
				}
				AlternateInfluence.KeepAlternateSkinningBuildDataOnly();
			}
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SkeletalMesh->SaveLODImportedData(LodIndex, ImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		if (bInResetReductionAsNeeded)
		{
			FSkeletalMeshOptimizationSettings& ReductionSettings = ThisLODInfo->ReductionSettings;
			
			//Remove the reduction settings
			ReductionSettings.NumOfTrianglesPercentage = 1.0f;
			ReductionSettings.NumOfVertPercentage = 1.0f;
			ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
			ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
			ReductionSettings.TerminationCriterion = SMTC_NumOfTriangles;
			ThisLODInfo->bHasBeenSimplified = false;
		}
		
		UE_ASSET_LOG(LogSkeletalMesh, Display, SkeletalMesh, TEXT("Import data converted for LOD %d"), LodIndex);
	}
}

void FMeshUtilities::AddMakeStaticMeshEntryToToolMenu(FName InToolMenuName)
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(InToolMenuName);
	FToolMenuSection& SkeletalMeshSection = ToolMenu->AddSection("SkeletalMesh", LOCTEXT("SkeletalMeshSection", "Skeletal Mesh"));

	SkeletalMeshSection.AddEntry(FToolMenuEntry::InitToolBarButton(
        "MakeStaticMesh",
        FToolUIActionChoice(
            FToolUIAction(FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
            {
            	if(UPersonaToolMenuContext* MenuContext = InContext.FindContext<UPersonaToolMenuContext>())
            	{
            		if(UDebugSkelMeshComponent* MeshComponent = MenuContext->GetPreviewMeshComponent())
            		{
            			ConvertMeshesToStaticMesh(TArray<UMeshComponent*>({ MeshComponent }), MeshComponent->GetComponentToWorld());
            		}
            	}
            }))
        ),
        LOCTEXT("MakeStaticMesh", "Make Static Mesh"),
        LOCTEXT("MakeStaticMeshTooltip", "Make a new static mesh out of the preview's current pose."),
        FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ConvertToStaticMesh"))
	);
}

void FMeshUtilities::AddLevelViewportMenuExtender()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

	MenuExtenders.Add(FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FMeshUtilities::GetLevelViewportContextMenuExtender));
	LevelViewportExtenderHandle = MenuExtenders.Last().GetHandle();
}

void FMeshUtilities::RemoveLevelViewportMenuExtender()
{
	if (LevelViewportExtenderHandle.IsValid())
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor");
		if (LevelEditorModule)
		{
			typedef FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors DelegateType;
			LevelEditorModule->GetAllLevelViewportContextMenuExtenders().RemoveAll([this](const DelegateType& In) { return In.GetHandle() == LevelViewportExtenderHandle; });
		}
	}
}

/** Util for getting all MeshComponents from a supplied set of Actors */
void GetSkinnedAndStaticMeshComponentsFromActors(const TArray<AActor*> InActors, TArray<UMeshComponent*>& OutMeshComponents)
{
	for (AActor* Actor : InActors)
	{
		// add all components from this actor
		TInlineComponentArray<UMeshComponent*> ActorComponents(Actor);
		for (UMeshComponent* ActorComponent : ActorComponents)
		{
			if (ActorComponent->IsA(USkinnedMeshComponent::StaticClass()) || ActorComponent->IsA(UStaticMeshComponent::StaticClass()))
			{
				OutMeshComponents.AddUnique(ActorComponent);
			}
		}

		// add all attached actors
		TArray<AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);
		for (AActor* AttachedActor : AttachedActors)
		{
			TInlineComponentArray<UMeshComponent*> AttachedActorComponents(AttachedActor);
			for (UMeshComponent* AttachedActorComponent : AttachedActorComponents)
			{
				if (AttachedActorComponent->IsA(USkinnedMeshComponent::StaticClass()) || AttachedActorComponent->IsA(UStaticMeshComponent::StaticClass()))
				{
					OutMeshComponents.AddUnique(AttachedActorComponent);
				}
			}
		}
	}
}

TSharedRef<FExtender> FMeshUtilities::GetLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors)
{
	TSharedRef<FExtender> Extender = MakeShareable(new FExtender);

	if (InActors.Num() > 0)
	{
		FText ActorName = InActors.Num() == 1 ? FText::Format(LOCTEXT("ActorNameSingular", "\"{0}\""), FText::FromString(InActors[0]->GetActorLabel())) : LOCTEXT("ActorNamePlural", "Actors");

		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedRef<FUICommandList> LevelEditorCommandBindings = LevelEditor.GetGlobalLevelEditorActions();

		// Note: ActorConvert extension point appears only in the pulldown Actor menu.
		Extender->AddMenuExtension("ActorConvert", EExtensionHook::After, LevelEditorCommandBindings, FMenuExtensionDelegate::CreateLambda(
			[this, ActorName, InActors](FMenuBuilder& MenuBuilder) {

			TArray<UMeshComponent*> Components;
			GetSkinnedAndStaticMeshComponentsFromActors(InActors, Components);

			const bool bCanExecute = Components.Num() > 0;

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("ConvertSelectedActorsToStaticMeshText", "Convert {0} To Static Mesh"), ActorName),
				LOCTEXT("ConvertSelectedActorsToStaticMeshTooltip", "Convert the selected actor's meshes to a new Static Mesh asset. Supports static and skeletal meshes."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FMeshUtilities::ConvertActorMeshesToStaticMeshUIAction, InActors),
					FCanExecuteAction::CreateLambda([bCanExecute]() { return bCanExecute; }))
			);
		}));
	}

	return Extender;
}

void FMeshUtilities::ConvertActorMeshesToStaticMeshUIAction(const TArray<AActor*> InActors)
{
	TArray<UMeshComponent*> MeshComponents;

	GetSkinnedAndStaticMeshComponentsFromActors(InActors, MeshComponents);

	auto GetActorRootTransform = [](AActor* InActor)
	{
		FTransform RootTransform(FTransform::Identity);
		if (ACharacter* Character = Cast<ACharacter>(InActor))
		{
			RootTransform = Character->GetTransform();
			RootTransform.SetLocation(RootTransform.GetLocation() - FVector(0.0f, 0.0f, Character->GetCapsuleComponent()->GetScaledCapsuleHalfHeight()));
		}
		else
		{
			// otherwise just use the actor's origin
			RootTransform = InActor->GetTransform();
		}

		return RootTransform;
	};

	// now pick a root transform
	FTransform RootTransform(FTransform::Identity);
	if (InActors.Num() == 1)
	{
		RootTransform = GetActorRootTransform(InActors[0]);
	}
	else
	{
		// multiple actors use the average of their origins, with Z being the min of all origins. Rotation is identity for simplicity
		FVector Location(FVector::ZeroVector);
		double MinZ = DBL_MAX;
		for (AActor* Actor : InActors)
		{
			FTransform ActorTransform(GetActorRootTransform(Actor));
			Location += ActorTransform.GetLocation();
			MinZ = FMath::Min(ActorTransform.GetLocation().Z, MinZ);
		}
		Location /= (float)InActors.Num();
		Location.Z = MinZ;

		RootTransform.SetLocation(Location);
	}

	UStaticMesh* StaticMesh = ConvertMeshesToStaticMesh(MeshComponents, RootTransform);

	// Also notify the content browser that the new assets exists
	if (StaticMesh != nullptr)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(TArray<UObject*>({ StaticMesh }), true);
	}
}

bool FMeshUtilities::GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2f>& OutTexCoords) const
{
	// Create a copy of original mesh (only copy necessary data)
	FRawMesh TempMesh;
	TempMesh.VertexPositions = RawMesh.VertexPositions;

	// Remove all duplicate faces if we are merging identical materials
	const int32 NumFaces = RawMesh.FaceMaterialIndices.Num();
	TArray<int32> DuplicateFaceRecords;
	
	if(bMergeIdenticalMaterials)
	{
		TArray<int32> UniqueFaceIndices;
		UniqueFaceIndices.Reserve(NumFaces);
		DuplicateFaceRecords.SetNum(NumFaces);

		TempMesh.WedgeTexCoords[0].Reserve(RawMesh.WedgeTexCoords[0].Num());
		TempMesh.WedgeIndices.Reserve(RawMesh.WedgeIndices.Num());

		// insert only non-duplicate faces
		for(int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
		{
			bool bFound = false;
			int32 UniqueFaceIndex = 0;
			for( ; UniqueFaceIndex < UniqueFaceIndices.Num(); ++UniqueFaceIndex)
			{
				int32 TestIndex = UniqueFaceIndices[UniqueFaceIndex];

				if (TestIndex != FaceIndex &&
					RawMesh.FaceMaterialIndices[FaceIndex] == RawMesh.FaceMaterialIndices[TestIndex] &&
					RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 0] == RawMesh.WedgeTexCoords[0][(TestIndex * 3) + 0] &&
					RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 1] == RawMesh.WedgeTexCoords[0][(TestIndex * 3) + 1] &&
					RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 2] == RawMesh.WedgeTexCoords[0][(TestIndex * 3) + 2])
				{
					bFound = true;
					break;
				}
			}

			if(!bFound)
			{
				UniqueFaceIndices.Add(FaceIndex);
				TempMesh.WedgeTexCoords[0].Add(RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 0]);
				TempMesh.WedgeTexCoords[0].Add(RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 1]);
				TempMesh.WedgeTexCoords[0].Add(RawMesh.WedgeTexCoords[0][(FaceIndex * 3) + 2]);
				TempMesh.WedgeIndices.Add(RawMesh.WedgeIndices[(FaceIndex * 3) + 0]);
				TempMesh.WedgeIndices.Add(RawMesh.WedgeIndices[(FaceIndex * 3) + 1]);
				TempMesh.WedgeIndices.Add(RawMesh.WedgeIndices[(FaceIndex * 3) + 2]);

				DuplicateFaceRecords[FaceIndex] = UniqueFaceIndices.Num() - 1;
			}
			else
			{
				DuplicateFaceRecords[FaceIndex] = UniqueFaceIndex;
			}
		}
	}
	else
	{
		TempMesh.WedgeTexCoords[0] = RawMesh.WedgeTexCoords[0];
		TempMesh.WedgeIndices = RawMesh.WedgeIndices;	
	}

	// Find overlapping corners for UV generator. Allow some threshold - this should not produce any error in a case if resulting
	// mesh will not merge these vertices.
	FOverlappingCorners OverlappingCorners;
	FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities").FindOverlappingCorners(OverlappingCorners, TempMesh.VertexPositions, TempMesh.WedgeIndices, THRESH_POINTS_ARE_SAME);

	// Generate new UVs
	FLayoutUVRawMeshView TempMeshView(TempMesh, 0, 1);
	FLayoutUV Packer(TempMeshView);
	Packer.FindCharts(OverlappingCorners);

	bool bPackSuccess = Packer.FindBestPacking(FMath::Clamp(TextureResolution / 4, 32, 512));
	if (bPackSuccess)
	{
		Packer.CommitPackedUVs();

		if(bMergeIdenticalMaterials)
		{
			// re-duplicate faces
			OutTexCoords.SetNum(RawMesh.WedgeTexCoords[0].Num());

			for(int32 FaceIndex = 0; FaceIndex < DuplicateFaceRecords.Num(); ++FaceIndex)
			{
				int32 SourceFaceIndex = DuplicateFaceRecords[FaceIndex];

				OutTexCoords[(FaceIndex * 3) + 0] = TempMesh.WedgeTexCoords[1][(SourceFaceIndex * 3) + 0];
				OutTexCoords[(FaceIndex * 3) + 1] = TempMesh.WedgeTexCoords[1][(SourceFaceIndex * 3) + 1];
				OutTexCoords[(FaceIndex * 3) + 2] = TempMesh.WedgeTexCoords[1][(SourceFaceIndex * 3) + 2];
			}
		}
		else
		{
			// Save generated UVs
			OutTexCoords = TempMesh.WedgeTexCoords[1];	
		}
 	}

	return bPackSuccess;
}

bool FMeshUtilities::GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const
{
	return GenerateUniqueUVsForStaticMesh(RawMesh, TextureResolution, false, OutTexCoords);
}

#undef LOCTEXT_NAMESPACE
