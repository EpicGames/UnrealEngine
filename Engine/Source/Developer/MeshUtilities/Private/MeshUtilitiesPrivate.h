// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshUtilities.h"
#include "SkeletalMeshTools.h"
#include "Engine/SkeletalMesh.h"
#include "IAnimationBlueprintEditor.h"
#include "IAnimationBlueprintEditorModule.h"
#include "IAnimationEditor.h"
#include "IAnimationEditorModule.h"
#include "ISkeletalMeshEditor.h"
#include "ISkeletalMeshEditorModule.h"
#include "ISkeletonEditor.h"
#include "ISkeletonEditorModule.h"
#include "Engine/StaticMesh.h"

class FMeshUtilities : public IMeshUtilities
{
private:
	/** Cached version string. */
	FString VersionString;
	/** True if NvTriStrip is being used for tri order optimization. */
	bool bUsingNvTriStrip;
	/** True if we disable triangle order optimization.  For debugging purposes only */
	bool bDisableTriangleOrderOptimization;

	// IMeshUtilities interface.
	virtual const FString& GetVersionString() const override
	{
		return VersionString;
	}

	virtual void FixupMaterialSlotNames(UStaticMesh* StaticMesh) const override;

	virtual void FixupMaterialSlotNames(USkeletalMesh* SkeletalMesh) const override;

	virtual bool BuildStaticMesh(
		FStaticMeshRenderData& OutRenderData,
		UStaticMesh* StaticMesh,
		const FStaticMeshLODGroup& LODGroup
		) override;

	virtual void BuildStaticMeshVertexAndIndexBuffers(
		TArray<FStaticMeshBuildVertex>& OutVertices,
		TArray<TArray<uint32> >& OutPerSectionIndices,
		TArray<int32>& OutWedgeMap,
		const FRawMesh& RawMesh,
		const FOverlappingCorners& OverlappingCorners,
		const TMap<uint32, uint32>& MaterialToSectionMapping,
		float ComparisonThreshold,
		FVector3f BuildScale,
		int32 ImportVersion
		) override;

	virtual bool GenerateStaticMeshLODs(UStaticMesh* StaticMesh, const FStaticMeshLODGroup& LODGroup) override;

	
	virtual int64_t MemoryForGenerateSignedDistanceFieldVolumeData(
		const FMeshDataForDerivedDataTask& MeshData,
		float DistanceFieldResolutionScale
	);

	/** Builds a signed distance field volume for the given mesh. */
	virtual void GenerateSignedDistanceFieldVolumeData(
		FString MeshName,
		const FMeshDataForDerivedDataTask& MeshData,
		float DistanceFieldResolutionScale,
		bool bGenerateAsIfTwoSided,
		class FDistanceFieldVolumeData& OutData) override;
	
	virtual int64 MemoryForGenerateMeshCardRepresentation(uint64_t IndexCount);

	virtual bool GenerateCardRepresentationData(
		FString MeshName,
		const FMeshDataForDerivedDataTask& MeshData,
		const FDistanceFieldVolumeData* DistanceFieldVolumeData,
		int32 MaxLumenMeshCards,
		bool bGenerateAsIfTwoSided,
		class FCardRepresentationData& OutData) override;

	virtual void RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, FRawMesh &OutRawMesh) const override;
	virtual void RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, const FOverlappingCorners& InOverlappingCorners, FRawMesh &OutRawMesh) const override;

	virtual bool GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const override;
	virtual bool GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2f>& OutTexCoords) const override;

	virtual bool BuildSkeletalMesh(FSkeletalMeshLODModel& LODModel,	const FString& SkeletalMeshName, const FReferenceSkeleton& RefSkeleton, const TArray<SkeletalMeshImportData::FVertInfluence>& Influences, const TArray<SkeletalMeshImportData::FMeshWedge>& Wedges, const TArray<SkeletalMeshImportData::FMeshFace>& Faces, const TArray<FVector3f>& Points, const TArray<int32>& PointToOriginalMap, const MeshBuildOptions& BuildOptions = MeshBuildOptions(), TArray<FText> * OutWarningMessages = nullptr, TArray<FName> * OutWarningNames = nullptr) override;
	
	virtual void CacheOptimizeIndexBuffer(TArray<uint16>& Indices) override;
	virtual void CacheOptimizeIndexBuffer(TArray<uint32>& Indices) override;
	void CacheOptimizeVertexAndIndexBuffer(TArray<FStaticMeshBuildVertex>& Vertices, TArray<TArray<uint32> >& PerSectionIndices, TArray<int32>& WedgeMap);

	virtual void BuildSkeletalAdjacencyIndexBuffer(
		const TArray<FSoftSkinVertex>& VertexBuffer,
		const uint32 TexCoordCount,
		const TArray<uint32>& Indices,
		TArray<uint32>& OutPnAenIndices
		) override;

	/**
	 *  Calculate The tangent bi normal and normal for the triangle define by the tree SoftSkinVertex.
	 *
	 *  @note The function will always fill properly the OutTangents array with 3 FVector. If the triangle is degenerated the OutTangent will contain zeroed vectors.
	 *
	 *  @param VertexA - First triangle vertex.
	 *  @param VertexB - Second triangle vertex.
	 *  @param VertexC - Third triangle vertex.
	 *  @param OutTangents - The function allocate the TArray with 3 FVector, to represent the triangle tangent, bi normal and normal.
	 *  @param CompareThreshold - The threshold use to compare a tangent vector with zero.
	 */
	virtual void CalculateTriangleTangent(const FSoftSkinVertex& VertexA, const FSoftSkinVertex& VertexB, const FSoftSkinVertex& VertexC, TArray<FVector3f>& OutTangents, float CompareThreshold) override;

	virtual void CalcBoneVertInfos(USkeletalMesh* SkeletalMesh, TArray<FBoneVertInfo>& Infos, bool bOnlyDominant, int32 LodIndex = 0) override;

	/** 
	 * Convert a set of mesh components in their current pose to a static mesh. 
	 * @param	InMeshComponents		The mesh components we want to convert
	 * @param	InRootTransform			The transform of the root of the mesh we want to output
	 * @param	InPackageName			The package name to create the static mesh in. If this is empty then a dialog will be displayed to pick the mesh.
	 * @return a new static mesh (specified by the user)
	 */
	virtual UStaticMesh* ConvertMeshesToStaticMesh(const TArray<UMeshComponent*>& InMeshComponents, const FTransform& InRootTransform = FTransform::Identity, const FString& InPackageName = FString()) override;

	/**
	* Builds a renderable skeletal mesh LOD model. Note that the array of chunks
	* will be destroyed during this process!
	* @param LODModel				Upon return contains a renderable skeletal mesh LOD model.
	* @param RefSkeleton			The reference skeleton associated with the model.
	* @param Chunks				Skinned mesh chunks from which to build the renderable model.
	* @param PointToOriginalMap	Maps a vertex's RawPointIdx to its index at import time.
	*/
	void BuildSkeletalModelFromChunks(FSkeletalMeshLODModel& LODModel, const FReferenceSkeleton& RefSkeleton, TArray<FSkinnedMeshChunk*>& Chunks, const TArray<int32>& PointToOriginalMap, bool bCacheOptimize = true);

	virtual void FindOverlappingCorners(FOverlappingCorners& OutOverlappingCorners, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, float ComparisonThreshold) const override;

	void FindOverlappingCorners(FOverlappingCorners& OutOverlappingCorners, FRawMesh const& RawMesh, float ComparisonThreshold) const;
	// IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void ExtractMeshDataForGeometryCache(FRawMesh& RawMesh, const FMeshBuildSettings& BuildSettings, TArray<FStaticMeshBuildVertex>& OutVertices, TArray<TArray<uint32> >& OutPerSectionIndices, int32 ImportVersion);

	virtual void CalculateTextureCoordinateBoundsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, TArray<FBox2D>& OutBounds) const override;

	virtual bool GenerateUniqueUVsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const override;

	virtual bool RemoveBonesFromMesh(USkeletalMesh* SkeletalMesh, int32 LODIndex, const TArray<FName>* BoneNamesToRemove) const override;
	virtual bool GetBoneReductionData(const USkeletalMesh* SkeletalMesh, int32 LODIndex, TMap<uint16, uint16>& OutBonesToReplace) const override;

	virtual void CalculateTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY, TArray<FVector3f>& OutNormals) const override;
	virtual void CalculateMikkTSpaceTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<FVector3f>& InNormals, bool bIgnoreDegenerateTriangles, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY) const override;
	virtual void CalculateNormals(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutNormals) const override;
	virtual void CalculateOverlappingCorners(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, bool bIgnoreDegenerateTriangles, FOverlappingCorners& OutOverlappingCorners) const override;

	virtual void GenerateRuntimeSkinWeightData(const FSkeletalMeshLODModel* ImportedModel, const TArray<FRawSkinWeight>& InRawSkinWeights, bool bInUseHighPrecisionWeights, FRuntimeSkinWeightProfileData& InOutSkinWeightOverrideData, const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices = nullptr) const override;

	virtual void CreateImportDataFromLODModel(USkeletalMesh* SkeletalMesh, bool bInResetReductionAsNeeded) const override;

	void RegisterMenus();

	// Need to call some members from this class, (which is internal to this module)
	friend class FStaticMeshUtilityBuilder;

protected:
	void AddMakeStaticMeshEntryToToolMenu(FName InToolbarName);

	void AddLevelViewportMenuExtender();

	void RemoveLevelViewportMenuExtender();

	TSharedRef<FExtender> GetLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors);

	void ConvertActorMeshesToStaticMeshUIAction(const TArray<AActor*> InActors);

	FDelegateHandle ModuleLoadedDelegateHandle;
	FDelegateHandle LevelViewportExtenderHandle;
	FDelegateHandle AnimationBlueprintEditorExtenderHandle;
	FDelegateHandle AnimationEditorExtenderHandle;
	FDelegateHandle SkeletonEditorExtenderHandle;
};

DECLARE_LOG_CATEGORY_EXTERN(LogMeshUtilities, Verbose, All);
namespace MeshUtilities
{
	/** Generates unit length, stratified and uniformly distributed direction samples in a hemisphere. */
	void GenerateStratifiedUniformHemisphereSamples(int32 NumSamples, FRandomStream& RandomStream, TArray<FVector3f>& Samples);
};