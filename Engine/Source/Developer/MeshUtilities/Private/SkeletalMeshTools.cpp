// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshTools.h"
#include "Engine/SkeletalMesh.h"
#include "MeshBuild.h"
#include "MeshUtilities.h"
#include "OverlappingCorners.h"
#include "RawIndexBuffer.h"
#include "Rendering/SkeletalMeshModel.h"

int32 GUseSkeletalMeshExperimentalChunking = 0;
static FAutoConsoleVariableRef CVarUseSkeletalMeshExperimentalChunking(
	TEXT("SkeletalMesh.UseExperimentalChunking"),
	GUseSkeletalMeshExperimentalChunking,
	TEXT("Whether skeletal mesh will use a experimental chunking algorithm when building LODModel.")
);

namespace SkeletalMeshTools
{
	bool AreSkelMeshVerticesEqual( const FSoftSkinBuildVertex& V1, const FSoftSkinBuildVertex& V2, const FOverlappingThresholds& OverlappingThresholds)
	{
		if(!PointsEqual(V1.Position, V2.Position, OverlappingThresholds))
		{
			return false;
		}

		for(int32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
		{
			if (!UVsEqual(V1.UVs[UVIdx], V2.UVs[UVIdx], OverlappingThresholds))
			{
				return false;
			}
		}

		if(!NormalsEqual(V1.TangentX, V2.TangentX, OverlappingThresholds))
		{
			return false;
		}

		if(!NormalsEqual(V1.TangentY, V2.TangentY, OverlappingThresholds))
		{
			return false;
		}

		if(!NormalsEqual(V1.TangentZ, V2.TangentZ, OverlappingThresholds))
		{
			return false;
		}

		bool	InfluencesMatch = 1;
		for(uint32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
		{
			if(V1.InfluenceBones[InfluenceIndex] != V2.InfluenceBones[InfluenceIndex] ||
				V1.InfluenceWeights[InfluenceIndex] != V2.InfluenceWeights[InfluenceIndex])
			{
				InfluencesMatch = 0;
				break;
			}
		}

		if (V1.Color != V2.Color)
		{
			return false;
		}

		if(!InfluencesMatch)
		{
			return false;
		}

		return true;
	}

	void BuildSkeletalMeshChunks( const TArray<SkeletalMeshImportData::FMeshFace>& Faces, const TArray<FSoftSkinBuildVertex>& RawVertices, TArray<FSkeletalMeshVertIndexAndZ>& RawVertIndexAndZ, const FOverlappingThresholds &OverlappingThresholds, TArray<FSkinnedMeshChunk*>& OutChunks, bool& bOutTooManyVerts )
	{
		FOverlappingCorners OverlappingCorners;
		OverlappingCorners.Init(RawVertIndexAndZ.Num());
		{

			// Sorting function for vertex Z/index pairs
			struct FCompareFSkeletalMeshVertIndexAndZ
			{
				FORCEINLINE bool operator()(const FSkeletalMeshVertIndexAndZ& A, const FSkeletalMeshVertIndexAndZ& B) const
				{
					return A.Z < B.Z;
				}
			};

			// Sort the vertices by z value
			RawVertIndexAndZ.Sort(FCompareFSkeletalMeshVertIndexAndZ());

			// Search for duplicates, quickly!
			for(int32 i = 0; i < RawVertIndexAndZ.Num(); i++)
			{
				// only need to search forward, since we add pairs both ways
				for(int32 j = i + 1; j < RawVertIndexAndZ.Num(); j++)
				{
					if(FMath::Abs(RawVertIndexAndZ[j].Z - RawVertIndexAndZ[i].Z) > OverlappingThresholds.ThresholdPosition)
					{
						// our list is sorted, so there can't be any more dupes
						break;
					}

					// check to see if the points are really overlapping
					if(PointsEqual(
						RawVertices[RawVertIndexAndZ[i].Index].Position,
						RawVertices[RawVertIndexAndZ[j].Index].Position, OverlappingThresholds))
					{
						OverlappingCorners.Add(RawVertIndexAndZ[i].Index, RawVertIndexAndZ[j].Index);
					}
				}
			}
		}
		OverlappingCorners.FinishAdding();

		TMap<FSkinnedMeshChunk* , TMap<int32, int32> > ChunkToFinalVerts;

	
		uint32 TriangleIndices[3];
		for(int32 FaceIndex = 0; FaceIndex < Faces.Num(); FaceIndex++)
		{
			const SkeletalMeshImportData::FMeshFace& Face = Faces[FaceIndex];

			// Find a chunk which matches this triangle.
			FSkinnedMeshChunk* Chunk = NULL;
			for(int32 i = 0; i < OutChunks.Num(); ++i)
			{
				if(OutChunks[i]->MaterialIndex == Face.MeshMaterialIndex)
				{
					Chunk = OutChunks[i];
					break;
				}
			}
			if(Chunk == NULL)
			{
				Chunk = new FSkinnedMeshChunk();
				Chunk->MaterialIndex = Face.MeshMaterialIndex;
				Chunk->OriginalSectionIndex = OutChunks.Num();
				OutChunks.Add(Chunk);
			}

			TMap<int32, int32>& FinalVerts = ChunkToFinalVerts.FindOrAdd( Chunk );

			for(int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
			{
				int32 WedgeIndex = FaceIndex * 3 + VertexIndex;
				const FSoftSkinBuildVertex& Vertex = RawVertices[WedgeIndex];

				int32 FinalVertIndex = INDEX_NONE;
				const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(WedgeIndex);

				for(int32 k = 0; k < DupVerts.Num(); k++)
				{
					if(DupVerts[k] >= WedgeIndex)
					{
						// the verts beyond me haven't been placed yet, so these duplicates are not relevant
						break;
					}

					int32 *Location = FinalVerts.Find(DupVerts[k]);
					if(Location != NULL)
					{
						if(SkeletalMeshTools::AreSkelMeshVerticesEqual(Vertex, Chunk->Vertices[*Location], OverlappingThresholds))
						{
							FinalVertIndex = *Location;
							break;
						}
					}
				}
				if(FinalVertIndex == INDEX_NONE)
				{
					FinalVertIndex = Chunk->Vertices.Add(Vertex);
					FinalVerts.Add(WedgeIndex, FinalVertIndex);
				}

				// set the index entry for the newly added vertex
				// TArray internally has int32 for capacity, so no need to test for uint32 as it's larger than int32
				TriangleIndices[VertexIndex] = (uint32)FinalVertIndex;
			}

			if(TriangleIndices[0] != TriangleIndices[1] && TriangleIndices[0] != TriangleIndices[2] && TriangleIndices[1] != TriangleIndices[2])
			{
				for(uint32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
				{
					Chunk->Indices.Add(TriangleIndices[VertexIndex]);
				}
			}
		}
	}


	namespace PolygonShellsHelper
	{
		struct FPatchAndBoneInfluence
		{
			TArray<FBoneIndexType> UniqueBones;
			TArray<int32> PatchToChunkWith;
			bool bIsParent = false;
		};
		//This function add every triangles connected to the triangle queue.
		//A connected triangle pair must share at least 1 vertex between the two triangles.
		//If bConnectByEdge is true, the connected triangle must share at least one edge (two vertex index)
		//To have a connected vertex instance pair, the position, NTBs, UVs(channel 0) and color must match.
		void AddAdjacentFace(const TArray<uint32>& Indices, const TArray<FSoftSkinBuildVertex>& Vertices, TBitArray<>& FaceAdded, const TMap<int32, TArray<int32>>& VertexIndexToAdjacentFaces, const int32 FaceIndex, TArray<int32>& TriangleQueue, const bool bConnectByEdge)
		{
			int32 NumFaces = Indices.Num()/3;
			check(FaceAdded.Num() == NumFaces);

			TMap<int32, int32> AdjacentFaceCommonVertices;
			for (int32 Corner = 0; Corner < 3; Corner++)
			{
				int32 IndiceIndex = FaceIndex * 3 + Corner;
				checkSlow(Indices.IsValidIndex(IndiceIndex));
				int32 VertexIndex = Indices[IndiceIndex];
				checkSlow(Vertices.IsValidIndex(VertexIndex));
				const FSoftSkinBuildVertex& SoftSkinVertRef = Vertices[VertexIndex];
				const FVector& PositionRef = (FVector)SoftSkinVertRef.Position;
				const FVector& TangentXRef = (FVector)SoftSkinVertRef.TangentX;
				const FVector& TangentYRef = (FVector)SoftSkinVertRef.TangentY;
				const FVector& TangentZRef = (FVector)SoftSkinVertRef.TangentZ;
				const FVector2f& UVRef = SoftSkinVertRef.UVs[0];
				const FColor& ColorRef = SoftSkinVertRef.Color;
				const TArray<int32>& AdjacentFaces = VertexIndexToAdjacentFaces.FindChecked(VertexIndex);
				for (int32 AdjacentFaceArrayIndex = 0; AdjacentFaceArrayIndex < AdjacentFaces.Num(); ++AdjacentFaceArrayIndex)
				{
					const int32 AdjacentFaceIndex = AdjacentFaces[AdjacentFaceArrayIndex];
					if (!FaceAdded[AdjacentFaceIndex] && AdjacentFaceIndex != FaceIndex)
					{
						//Ensure we have position, NTBs, uv and color match to allow a connection.
						bool bRealConnection = false;
						for (int32 AdjCorner = 0; AdjCorner < 3; AdjCorner++)
						{
							const int32 IndiceIndexAdj = AdjacentFaceIndex * 3 + AdjCorner;
							checkSlow(Indices.IsValidIndex(IndiceIndexAdj));
							const int32 VertexIndexAdj = Indices[IndiceIndexAdj];
							checkSlow(Vertices.IsValidIndex(VertexIndexAdj));
							const FSoftSkinBuildVertex& SoftSkinVertAdj = Vertices[VertexIndexAdj];
							if (PositionRef.Equals((FVector)SoftSkinVertAdj.Position, SMALL_NUMBER) &&
								TangentXRef.Equals((FVector)SoftSkinVertAdj.TangentX, SMALL_NUMBER) &&
								TangentYRef.Equals((FVector)SoftSkinVertAdj.TangentY, SMALL_NUMBER) &&
								TangentZRef.Equals((FVector)SoftSkinVertAdj.TangentZ, SMALL_NUMBER) &&
								UVRef.Equals(SoftSkinVertAdj.UVs[0], KINDA_SMALL_NUMBER) &&
								ColorRef == SoftSkinVertAdj.Color)
							{
								bRealConnection = true;
								break;
							}
						}
						if (!bRealConnection)
						{
							continue;
						}

						bool bAddConnected = !bConnectByEdge;
						if (bConnectByEdge)
						{
							int32& AdjacentFaceCommonVerticeCount = AdjacentFaceCommonVertices.FindOrAdd(AdjacentFaceIndex);
							AdjacentFaceCommonVerticeCount++;
							//Is the connected triangles share 2 vertex index (one edge) not only one vertex
							bAddConnected = AdjacentFaceCommonVerticeCount > 1;
						}

						if (bAddConnected)
						{
							TriangleQueue.Add(AdjacentFaceIndex);
							//Add the face only once by marking the face has computed
							FaceAdded[AdjacentFaceIndex] = true;
						}
					}
				}
			}
		}


		//Fill FaceIndexToPatchIndex so every triangle knows its unique island patch index.
		//Each island patch have is fill with connected vertexinstance where position, NTBs. UVs and colors are nearly equal.
		//@Param bConnectByEdge: If true we need at least 2 vertex index (one edge) to connect 2 triangles. If false we just need one vertex index (bowtie)
		void FillPolygonPatch(const TArray<uint32>& Indices, const TArray<FSoftSkinBuildVertex>& Vertices, const TMap<uint32, TArray<FBoneIndexType>>& AlternateBoneIDs, TArray<FPatchAndBoneInfluence>& PatchData, TArray<TArray<uint32>>& PatchIndexToIndices, TMap<int32, TArray<FBoneIndexType>>& BonesPerFace, const int32 MaxBonesPerChunk, const bool bConnectByEdge)
		{
			const int32 NumIndice = Indices.Num();
			const int32 NumFace = NumIndice / 3;
			int32 PatchIndex = 0;
			
			//Store a map containing connected faces for each vertex index
			TMap<int32, TArray<int32>> VertexIndexToAdjacentFaces;
			VertexIndexToAdjacentFaces.Reserve(Vertices.Num());
			//Store a map to retrieve bones use per face
			BonesPerFace.Reserve(NumFace);
			for (int32 FaceIndex = 0; FaceIndex < NumFace; ++FaceIndex)
			{
				const int32 IndiceOffset = FaceIndex * 3;
				TArray<FBoneIndexType>& FaceInfluenceBones = BonesPerFace.FindOrAdd(FaceIndex);
				for (int32 Corner = 0; Corner < 3; Corner++)
				{
					const int32 IndiceIndex = IndiceOffset + Corner;
					checkSlow(Indices.IsValidIndex(IndiceIndex));
					int32 VertexIndex = Indices[IndiceIndex];
					TArray<int32>& AdjacentFaces = VertexIndexToAdjacentFaces.FindOrAdd(VertexIndex);
					AdjacentFaces.AddUnique(FaceIndex);
					const FSoftSkinBuildVertex& SoftSkinVertex = Vertices[VertexIndex];
					for (int32 BoneIndex = 0; BoneIndex < MAX_TOTAL_INFLUENCES; ++BoneIndex)
					{
						if (SoftSkinVertex.InfluenceWeights[BoneIndex] > 0)
						{
							FaceInfluenceBones.AddUnique(SoftSkinVertex.InfluenceBones[BoneIndex]);
						}
					}
					//Add the alternate bones
					const TArray<FBoneIndexType>* AlternateBones = AlternateBoneIDs.Find(SoftSkinVertex.PointWedgeIdx);
					if (AlternateBones)
					{
						for (int32 InfluenceIndex = 0; InfluenceIndex < AlternateBones->Num(); InfluenceIndex++)
						{
							FaceInfluenceBones.AddUnique((*AlternateBones)[InfluenceIndex]);
						}
					}
				}
			}

			//Mark added face so we do not add them more then once
			TBitArray<> FaceAdded;
			FaceAdded.Init(false, NumFace);

			TArray<int32> TriangleQueue;
			TriangleQueue.Reserve(100);
			//Allocate an array and use it to retrieve the data, we do not know the number of indices per patch so it prevent us doing a huge reserve per patch
			//Simply copy the result in PatchIndexToIndices when we finish gathering the patch data.
			TArray<uint32> AllocatedPatchIndexToIndices;
			AllocatedPatchIndexToIndices.Reserve(NumIndice);
			for (int32 FaceIndex = 0; FaceIndex < NumFace; ++FaceIndex)
			{
				//Skip already added faces
				if (FaceAdded[FaceIndex])
				{
					continue;
				}
				AllocatedPatchIndexToIndices.Reset();

				//Add all the faces connected to the current face index
				TriangleQueue.Reset();
				TriangleQueue.Add(FaceIndex); //Use a queue to avoid recursive function
				FaceAdded[FaceIndex] = true;
				while (TriangleQueue.Num() > 0)
				{
					int32 CurrentTriangleIndex = TriangleQueue.Pop(EAllowShrinking::No);
					TArray<FBoneIndexType> BonesToAdd = BonesPerFace[CurrentTriangleIndex];
					for (const FBoneIndexType BoneIndex : BonesToAdd)
					{
						if (!PatchData.IsValidIndex(PatchIndex))
						{
							PatchData.AddDefaulted(PatchData.Num() - PatchIndex + 1);
						}
						PatchData[PatchIndex].UniqueBones.AddUnique(BoneIndex);
					}
					int32 IndiceOffset = CurrentTriangleIndex * 3;
					for (int32 Corner = 0; Corner < 3; Corner++)
					{
						const int32 IndiceIndex = IndiceOffset + Corner;
						AllocatedPatchIndexToIndices.Add(IndiceIndex);
					}
					//The patch should exist at this time
					checkSlow(PatchData.IsValidIndex(PatchIndex));

					AddAdjacentFace(Indices, Vertices, FaceAdded, VertexIndexToAdjacentFaces, CurrentTriangleIndex, TriangleQueue, bConnectByEdge);
				}

				//This is a new patch create the data and append the patch result remap
				check(!PatchIndexToIndices.IsValidIndex(PatchIndex));
				PatchIndexToIndices.AddDefaulted();
				check(PatchIndexToIndices.IsValidIndex(PatchIndex));
				PatchIndexToIndices[PatchIndex].Append(AllocatedPatchIndexToIndices);
				PatchIndex++;
			}
		}

		void RecursiveFillRemapIndices(const TArray<FPatchAndBoneInfluence>& PatchData, const int32 PatchIndex, const TArray<TArray<uint32>>& PatchIndexToIndices, TArray<uint32>& SrcChunkRemapIndicesIndex)
		{
			SrcChunkRemapIndicesIndex.Append(PatchIndexToIndices[PatchIndex]);
			checkSlow(PatchData.IsValidIndex(PatchIndex));
			//Do the child patch to chunk with
			for (int32 SubPatchIndex = 0; SubPatchIndex < PatchData[PatchIndex].PatchToChunkWith.Num(); ++SubPatchIndex)
			{
				RecursiveFillRemapIndices(PatchData, PatchData[PatchIndex].PatchToChunkWith[SubPatchIndex], PatchIndexToIndices, SrcChunkRemapIndicesIndex);
			}
		}

		//Sort the shells to a setup that use the less section possible
		void GatherShellUsingSameBones(const int32 ParentPatchIndex, TArray<FPatchAndBoneInfluence>& PatchData, TBitArray<>& PatchConsumed, const int32 MaxBonesPerChunk)
		{
			checkSlow(PatchData.IsValidIndex(ParentPatchIndex));
			TArray<FBoneIndexType> UniqueBones = PatchData[ParentPatchIndex].UniqueBones;
			PatchData[ParentPatchIndex].bIsParent = true;
			if (UniqueBones.Num() > MaxBonesPerChunk)
			{
				return;
			}
			for (int32 PatchIndex = ParentPatchIndex + 1; PatchIndex < PatchData.Num(); ++PatchIndex)
			{
				if (PatchConsumed[PatchIndex])
				{
					continue;
				}

				TArray<FBoneIndexType> AddedBones;
				for (int32 BoneIndex = 0; BoneIndex < PatchData[PatchIndex].UniqueBones.Num(); ++BoneIndex)
				{
					FBoneIndexType BoneIndexType = PatchData[PatchIndex].UniqueBones[BoneIndex];
					if (!UniqueBones.Contains(BoneIndexType))
					{
						AddedBones.AddUnique(BoneIndexType);
					}
				}
				if (AddedBones.Num() + UniqueBones.Num() <= MaxBonesPerChunk)
				{
					UniqueBones.Append(AddedBones);
					PatchConsumed[PatchIndex] = true;
					//We only support one parent layer, the assumption is we have a hierarchy depth of max 2 (parents, childs)
					checkSlow(!PatchData[PatchIndex].bIsParent);
					PatchData[ParentPatchIndex].PatchToChunkWith.Add(PatchIndex);
				}
			}
		}
	}

	void ChunkSkinnedVertices(TArray<FSkinnedMeshChunk*>& Chunks, TMap<uint32, TArray<FBoneIndexType>>& AlternateBoneIDs, int32 MaxBonesPerChunk)
	{
#if WITH_EDITORONLY_DATA
		//Get the cvar that drive if we use the experimental chunking
		bool bUseExperimentalChunking = GUseSkeletalMeshExperimentalChunking != 0;

		// Copy over the old chunks (this is just copying pointers).
		TArray<FSkinnedMeshChunk*> SrcChunks;
		Exchange(Chunks,SrcChunks);

		// Sort the chunks by material index.
		struct FCompareSkinnedMeshChunk
		{
			FORCEINLINE bool operator()(const FSkinnedMeshChunk& A,const FSkinnedMeshChunk& B) const
			{
				return A.MaterialIndex < B.MaterialIndex;
			}
		};
		SrcChunks.Sort(FCompareSkinnedMeshChunk());

		TMap<int32, TArray<PolygonShellsHelper::FPatchAndBoneInfluence>> PatchDataPerSrcChunk;
		TMap<int32, TArray<TArray<uint32>>> PatchIndexToIndicesPerSrcChunk;
		TMap<int32, TMap<int32, TArray<FBoneIndexType>>> PatchIndexToBonesPerFace;
		
		//Find the shells inside chunks
		for (int32 ChunkIndex = 0; ChunkIndex < SrcChunks.Num(); ++ChunkIndex)
		{
			FSkinnedMeshChunk* ChunkToShell = SrcChunks[ChunkIndex];
			TArray<uint32>& Indices = ChunkToShell->Indices;
			TArray<FSoftSkinBuildVertex>& Vertices = ChunkToShell->Vertices;
			TArray<PolygonShellsHelper::FPatchAndBoneInfluence>& PatchData = PatchDataPerSrcChunk.Add(ChunkIndex);
			TArray<TArray<uint32>>& PatchIndexToIndices = PatchIndexToIndicesPerSrcChunk.Add(ChunkIndex);
			TMap<int32, TArray<FBoneIndexType>>& BonesPerFace = PatchIndexToBonesPerFace.Add(ChunkIndex);
			//We need edge connection (2 similar vertex )
			const bool bConnectByEdge = true;
			PolygonShellsHelper::FillPolygonPatch(Indices, Vertices, AlternateBoneIDs, PatchData, PatchIndexToIndices, BonesPerFace, MaxBonesPerChunk, bConnectByEdge);
		}

		for (int32 SrcChunkIndex = 0; SrcChunkIndex < SrcChunks.Num(); ++SrcChunkIndex)
		{
			TArray<PolygonShellsHelper::FPatchAndBoneInfluence>& PatchData = PatchDataPerSrcChunk[SrcChunkIndex];
			TBitArray<> PatchConsumed;
			PatchConsumed.Init(false, PatchData.Num());

			for (int32 PatchIndex = 0; PatchIndex < PatchData.Num(); ++PatchIndex)
			{
				if (PatchConsumed[PatchIndex])
				{
					continue;
				}
				PatchConsumed[PatchIndex] = true;
				PolygonShellsHelper::GatherShellUsingSameBones(PatchIndex, PatchData, PatchConsumed, MaxBonesPerChunk);
			}
		}

		// Now split chunks to respect the desired bone limit.
		TIndirectArray<TArray<int32> > IndexMaps;
		for (int32 SrcChunkIndex = 0; SrcChunkIndex < SrcChunks.Num(); ++SrcChunkIndex)
		{
			FSkinnedMeshChunk* SrcChunk = SrcChunks[SrcChunkIndex];
			SrcChunk->OriginalSectionIndex = SrcChunkIndex;
			int32 FirstChunkIndex = Chunks.Num();
			//Iterate Indice in the order of the shell patch
			TArray<uint32> SrcChunkRemapIndicesIndex;
			SrcChunkRemapIndicesIndex.Reserve(SrcChunk->Indices.Num());
			TArray<PolygonShellsHelper::FPatchAndBoneInfluence>& PatchData = PatchDataPerSrcChunk[SrcChunkIndex];
			const TArray<TArray<uint32>>& PatchIndexToIndices = PatchIndexToIndicesPerSrcChunk[SrcChunkIndex];
			const TMap<int32, TArray<FBoneIndexType>>& BonesPerFace = PatchIndexToBonesPerFace[SrcChunkIndex];

			for (int32 PatchIndex = 0; PatchIndex < PatchData.Num(); ++PatchIndex)
			{
				if (!PatchData[PatchIndex].bIsParent)
				{
					continue;
				}
				SrcChunkRemapIndicesIndex.Reset();
				PolygonShellsHelper::RecursiveFillRemapIndices(PatchData, PatchIndex, PatchIndexToIndices, SrcChunkRemapIndicesIndex);

				//Force adding a chunk since we want to control where we cut the model
				int32 LastCreatedChunkIndex = FirstChunkIndex;
				const int32 PatchInitialChunkIndex = Chunks.Num();

				auto CreateChunk = [&SrcChunk, &FirstChunkIndex, &LastCreatedChunkIndex, &Chunks, &IndexMaps](FSkinnedMeshChunk** DestinationChunk)
				{
					(*DestinationChunk) = new FSkinnedMeshChunk();
					LastCreatedChunkIndex = Chunks.Add(*DestinationChunk);
					(*DestinationChunk)->MaterialIndex = SrcChunk->MaterialIndex;
					(*DestinationChunk)->OriginalSectionIndex = SrcChunk->OriginalSectionIndex;
					(*DestinationChunk)->ParentChunkSectionIndex = LastCreatedChunkIndex == FirstChunkIndex ? INDEX_NONE : FirstChunkIndex;
					TArray<int32>& IndexMap = *new TArray<int32>();
					IndexMaps.Add(&IndexMap);
					IndexMap.AddUninitialized(SrcChunk->Vertices.Num());
					FMemory::Memset(IndexMap.GetData(), 0xff, IndexMap.GetTypeSize()*IndexMap.Num());
				};
				
				//Create a chunk
				{
					FSkinnedMeshChunk* DestChunk = NULL;
					CreateChunk(&DestChunk);
				}

				//Add Indices to the chunk and add extra chunk only in case the patch use more bone then the maximum specified
				for (int32 i = 0; i < SrcChunkRemapIndicesIndex.Num(); i += 3)
				{
					//We remap the iteration order to avoid cutting polygon shell
					int32 IndiceIndex = SrcChunkRemapIndicesIndex[i];
					// Find all bones needed by this triangle.
					const int32 FaceIndex = (IndiceIndex / 3);
					const TArray<FBoneIndexType>& UniqueBones = BonesPerFace.FindChecked(FaceIndex);

					// Now find a chunk for them.
					FSkinnedMeshChunk* DestChunk = NULL;
					int32 DestChunkIndex = bUseExperimentalChunking ? PatchInitialChunkIndex : LastCreatedChunkIndex;
					int32 SmallestNumBoneToAdd = MAX_int32;
					for (int32 ChunkIndex = DestChunkIndex; ChunkIndex < Chunks.Num(); ++ChunkIndex)
					{
						const TArray<FBoneIndexType>& BoneMap = Chunks[ChunkIndex]->BoneMap;
						int32 NumUniqueBones = 0;
						for (int32 j = 0; j < UniqueBones.Num(); ++j)
						{
							NumUniqueBones += (BoneMap.Contains(UniqueBones[j]) ? 0 : 1);
							if (NumUniqueBones == SmallestNumBoneToAdd)
							{
								//Another previous chunk use less or equal unique bone, avoid searching more
								break;
							}
						}
						if (NumUniqueBones + BoneMap.Num() <= MaxBonesPerChunk && NumUniqueBones < SmallestNumBoneToAdd)
						{
							//Add the vertex to the chunk that can contain it with the less addition.
							SmallestNumBoneToAdd = NumUniqueBones;
							DestChunkIndex = ChunkIndex;
							DestChunk = Chunks[ChunkIndex];
							if (SmallestNumBoneToAdd == 0)
							{
								//This is the best candidate
								break;
							}
						}
					}

					// If no chunk was found, create one!
					if (DestChunk == NULL)
					{
						CreateChunk(&DestChunk);
						//Set back the DestChunkIndex. CreateChunk set the LastCreatedChunkIndex, so we need to update DestChunkIndex to pick
						//The right IndexMaps that match the new chunk.
						DestChunkIndex = LastCreatedChunkIndex;
					}
					TArray<int32>& IndexMap = IndexMaps[DestChunkIndex];

					// Add the unique bones to this chunk's bone map.
					for (int32 j = 0; j < UniqueBones.Num(); ++j)
					{
						DestChunk->BoneMap.AddUnique(UniqueBones[j]);
					}

					// For each vertex, add it to the chunk's arrays of vertices and indices.
					for (int32 Corner = 0; Corner < 3; Corner++)
					{
						int32 VertexIndex = SrcChunk->Indices[IndiceIndex + Corner];
						int32 DestIndex = IndexMap[VertexIndex];
						if (DestIndex == INDEX_NONE)
						{
							DestIndex = DestChunk->Vertices.Add(SrcChunk->Vertices[VertexIndex]);
							FSoftSkinBuildVertex& V = DestChunk->Vertices[DestIndex];
							for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; InfluenceIndex++)
							{
								if (V.InfluenceWeights[InfluenceIndex] > 0)
								{
									int32 MappedIndex = DestChunk->BoneMap.Find(V.InfluenceBones[InfluenceIndex]);
									checkSlow(DestChunk->BoneMap.IsValidIndex(MappedIndex));
									V.InfluenceBones[InfluenceIndex] = MappedIndex;
								}
							}
							IndexMap[VertexIndex] = DestIndex;
						}
						DestChunk->Indices.Add(DestIndex);
					}
				}
			}


			// Source chunks are no longer needed.
			delete SrcChunks[SrcChunkIndex];
			SrcChunks[SrcChunkIndex] = NULL;
		}
#endif // #if WITH_EDITORONLY_DATA
	}
}

