// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Components.h"
#include "Containers/IndirectArray.h"
#include "CoreMinimal.h"
#include "GPUSkinPublicDefs.h"
#include "MeshBuild.h"
#include "Rendering/SkeletalMeshLODImporterData.h"

class USkeletalMesh;
struct FSoftSkinVertex;
struct FSkelMeshSection;
struct FBoneVertInfo;

// this is used for a sub-quadratic routine to find "equal" verts
struct FSkeletalMeshVertIndexAndZ
{
	int32 Index;
	float Z;
};

struct FSoftSkinBuildVertex
{
	FVector3f		Position;
	FVector3f		TangentX,	// Tangent, U-direction
					TangentY,	// Binormal, V-direction
					TangentZ;	// Normal
	FVector2f		UVs[MAX_TEXCOORDS]; // UVs
	FColor			Color;		// VertexColor
	FBoneIndexType	InfluenceBones[MAX_TOTAL_INFLUENCES];
	uint16			InfluenceWeights[MAX_TOTAL_INFLUENCES];
	uint32 PointWedgeIdx;
};

/**
 * A chunk of skinned mesh vertices used as intermediate data to build a renderable
 * skinned mesh.
 */
struct FSkinnedMeshChunk
{
	/** The material index with which this chunk should be rendered. */
	int32 MaterialIndex;
	/** The original section index for which this chunk was generated. */
	int32 OriginalSectionIndex;
	/** The vertices associated with this chunk. */
	TArray<FSoftSkinBuildVertex> Vertices;
	/** The indices of the triangles in this chunk. */
	TArray<uint32> Indices;
	/** If not empty, contains a map from bones referenced in this chunk to the skeleton. */
	TArray<FBoneIndexType> BoneMap;
	/** The parent original section index for which this chunk was generated. INDEX_NONE for parent the value of the parent for BONE child chunked*/
	int32 ParentChunkSectionIndex;
};

/**
 * Skinned model data needed to generate skinned mesh chunks for reprocessing.
 */
struct FSkinnedModelData
{
	/** Vertices of the model. */
	TArray<FSoftSkinVertex> Vertices;
	/** Indices of the model. */
	TArray<uint32> Indices;
	/** Contents of the model's RawPointIndices bulk data. */
	TArray<uint32> RawPointIndices;
	/** Map of vertex index to the original import index. */
	TArray<int32> MeshToImportVertexMap;
	/** Per-section information. */
	TArray<FSkelMeshSection> Sections;
	/** Per-section bone maps. */
	TIndirectArray<TArray<FBoneIndexType> > BoneMaps;
	/** The number of valid texture coordinates. */
	int32 NumTexCoords;
};

namespace SkeletalMeshTools
{
	inline bool SkeletalMesh_UVsEqual(const SkeletalMeshImportData::FMeshWedge& V1, const SkeletalMeshImportData::FMeshWedge& V2, const FOverlappingThresholds& OverlappingThresholds, const int32 UVIndex = 0)
	{
		const FVector2f& UV1 = V1.UVs[UVIndex];
		const FVector2f& UV2 = V2.UVs[UVIndex];

		if(FMath::Abs(UV1.X - UV2.X) > OverlappingThresholds.ThresholdUV)
			return 0;

		if(FMath::Abs(UV1.Y - UV2.Y) > OverlappingThresholds.ThresholdUV)
			return 0;

		return 1;
	}

	/** @return true if V1 and V2 are equal */
	bool AreSkelMeshVerticesEqual( const FSoftSkinBuildVertex& V1, const FSoftSkinBuildVertex& V2, const FOverlappingThresholds& OverlappingThresholds);

	/**
	 * Creates chunks and populates the vertex and index arrays inside each chunk
	 *
	 * @param Faces						List of raw faces
	 * @param RawVertices				List of raw created, unordered, unwelded vertices
	 * @param RawVertIndexAndZ			List of indices into the RawVertices array and each raw vertex Z position.  This is used for fast lookup of overlapping vertices
	 * @param OverlappingThresholds		The thresholds to use to compute overlap vertex instance
	 * @param OutChunks					Created array of chunks
	 */
	void BuildSkeletalMeshChunks( const TArray<SkeletalMeshImportData::FMeshFace>& Faces, const TArray<FSoftSkinBuildVertex>& RawVertices, TArray<FSkeletalMeshVertIndexAndZ>& RawVertIndexAndZ, const FOverlappingThresholds &OverlappingThresholds, TArray<FSkinnedMeshChunk*>& OutChunks, bool& bOutTooManyVerts );

	/**
	 * Splits chunks to satisfy the requested maximum number of bones per chunk
	 * @param Chunks			Chunks to split. Upon return contains the results of splitting chunks.
	 * @param MaxBonesPerChunk	The maximum number of bones a chunk may reference.
	*/
	void ChunkSkinnedVertices(TArray<FSkinnedMeshChunk*>& Chunks, TMap<uint32, TArray<FBoneIndexType>>& AlternateBoneIDs, int32 MaxBonesPerChunk);

};
