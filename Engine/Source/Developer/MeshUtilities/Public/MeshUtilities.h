// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Modules/ModuleInterface.h"
#include "Components.h"
#include "MeshBuild.h"

#include "DerivedMeshDataTaskUtils.h"

#include "IMeshMergeUtilities.h"

class UMeshComponent;
class USkeletalMesh;
class UStaticMesh;
class UStaticMeshComponent;
struct FFlattenMaterial;
struct FReferenceSkeleton;
struct FBoneVertInfo;
struct FRawMesh;
struct FRawSkinWeight;
struct FStaticMeshLODResources;
class FSkeletalMeshUnifiedBoneIndices;
class FSkeletalMeshLODModel;
class FSourceMeshDataForDerivedDataTask;

typedef FIntPoint FMeshIdAndLOD;

namespace SkeletalMeshImportData
{
	struct FMeshFace;
	struct FMeshWedge;
	struct FVertInfluence;
};

namespace ETangentOptions
{
	enum Type
	{
		None = 0,
		BlendOverlappingNormals = 0x1,
		IgnoreDegenerateTriangles = 0x2,
		UseMikkTSpace = 0x4,
	};
};

struct FSignedDistanceFieldBuildSectionData
{
	EBlendMode BlendMode = BLEND_Opaque;
	bool bTwoSided = false;
	bool bAffectDistanceFieldLighting = true;
};

struct FSignedDistanceFieldBuildPartData
{
	FSourceMeshDataForDerivedDataTask SourceMeshData;
	TArray<FSignedDistanceFieldBuildSectionData> SectionData;
};

struct FMeshDataForDerivedDataTask
{
	const FSourceMeshDataForDerivedDataTask* SourceMeshData;
	const FStaticMeshLODResources* LODModel;
	TConstArrayView<FSignedDistanceFieldBuildSectionData> SectionData;
	TConstArrayView<FSignedDistanceFieldBuildPartData> Parts;
	TConstArrayView<FNaniteAssemblyNode> Nodes;
	FBoxSphereBounds3f Bounds;
};

struct FOverlappingCorners;

class IMeshUtilities : public IModuleInterface
{
public:

	/**
	* Calculates (new) non-overlapping UV coordinates for the given Raw Mesh
	*
	* @param RawMesh - Raw Mesh to generate UV coordinates for
	* @param TextureResolution - Texture resolution to take into account while generating the UVs
	* @param bMergeIdenticalMaterials - Whether faces with identical materials can be treated as one in the resulting set of unique UVs
	* @param OutTexCoords - New set of UV coordinates
	* @return bool - whether or not generating the UVs succeeded
	*/
	virtual bool GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const = 0;
	virtual bool GenerateUniqueUVsForStaticMesh(const FRawMesh& RawMesh, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2f>& OutTexCoords) const = 0;
	
public:
	/** Returns a string uniquely identifying this version of mesh utilities. */
	virtual const FString& GetVersionString() const = 0;

	/** Used to make sure all imported material slot name are unique and non empty. 
	 * 
	 * @param StaticMesh
	 * @param bForceUniqueSlotName	If true, make sure all slot names are unique as well.
	 */
	virtual void FixupMaterialSlotNames(UStaticMesh* StaticMesh) const = 0;

	/** Used to make sure all imported material slot name are unique and non empty. 
	 *
	 * @param SkeletalMesh
	 * @param bForceUniqueSlotName	If true, make sure all slot names are unique as well.
	 */
	virtual void FixupMaterialSlotNames(USkeletalMesh* SkeletalMesh) const = 0;

	/**
	 * Builds a renderable static mesh using the provided source models and the LOD groups settings.
	 * @returns true if the renderable mesh was built successfully.
	 */
	virtual bool BuildStaticMesh(
		class FStaticMeshRenderData& OutRenderData,
		UStaticMesh* StaticMesh,
		const class FStaticMeshLODGroup& LODGroup
		) = 0;

	virtual void BuildStaticMeshVertexAndIndexBuffers(
		TArray<FStaticMeshBuildVertex>& OutVertices,
		TArray<TArray<uint32> >& OutPerSectionIndices,
		TArray<int32>& OutWedgeMap,
		const FRawMesh& RawMesh,
		const FOverlappingCorners& OverlappingCorners,
		const TMap<uint32, uint32>& MaterialToSectionMapping,
		float ComparisonThreshold,
		FVector3f BuildScale,
		int32 ImportVersion
		) = 0;

	/**
	 * Builds a static mesh using the provided source models and the LOD groups settings, and replaces
	 * the RawMeshes with the reduced meshes. Does not modify renderable data.
	 * @returns true if the meshes were built successfully.
	 */
	virtual bool GenerateStaticMeshLODs(
		UStaticMesh* StaticMesh,
		const class FStaticMeshLODGroup& LODGroup
		) = 0;

	/** Builds a signed distance field volume for the given mesh. */
	virtual void GenerateSignedDistanceFieldVolumeData(
		FString MeshName,
		const FMeshDataForDerivedDataTask& MeshData,
		float DistanceFieldResolutionScale,
		bool bGenerateAsIfTwoSided,
		class FDistanceFieldVolumeData& OutData) = 0;

	virtual int64_t MemoryForGenerateSignedDistanceFieldVolumeData(
		const FMeshDataForDerivedDataTask& MeshData,
		float DistanceFieldResolutionScale
	) = 0;

	UE_DEPRECATED(5.6, "Provide FMeshDataForDerivedDataTask instead.")
	virtual void GenerateSignedDistanceFieldVolumeData(
		FString MeshName,
		const FSourceMeshDataForDerivedDataTask& SourceMeshData,
		const FStaticMeshLODResources& LODModel,
		class FQueuedThreadPool& ThreadPool,
		const TArray<FSignedDistanceFieldBuildSectionData>& SectionData,
		const FBoxSphereBounds3f& Bounds,
		float DistanceFieldResolutionScale,
		bool bGenerateAsIfTwoSided,
		class FDistanceFieldVolumeData& OutData)
	{
		FMeshDataForDerivedDataTask MeshData;
		MeshData.SourceMeshData = &SourceMeshData;
		MeshData.LODModel = &LODModel;
		MeshData.SectionData = SectionData;
		MeshData.Bounds = Bounds;

		GenerateSignedDistanceFieldVolumeData(MeshName, MeshData, DistanceFieldResolutionScale, bGenerateAsIfTwoSided, OutData);
	}

	virtual int64 MemoryForGenerateMeshCardRepresentation(uint64_t IndexCount) = 0;

	virtual bool GenerateCardRepresentationData(
		FString MeshName,
		const FMeshDataForDerivedDataTask& MeshData,
		const class FDistanceFieldVolumeData* DistanceFieldVolumeData,
		int32 MaxLumenMeshCards,
		bool bGenerateAsIfTwoSided,
		class FCardRepresentationData& OutData) = 0;

	UE_DEPRECATED(5.6, "Provide FMeshDataForDerivedDataTask instead.")
	virtual bool GenerateCardRepresentationData(
		FString MeshName,
		const FSourceMeshDataForDerivedDataTask& SourceMeshData,
		const FStaticMeshLODResources& LODModel,
		class FQueuedThreadPool& ThreadPool,
		const TArray<FSignedDistanceFieldBuildSectionData>& SectionData,
		const FBoxSphereBounds& Bounds,
		const class FDistanceFieldVolumeData* DistanceFieldVolumeData,
		int32 MaxLumenMeshCards,
		bool bGenerateAsIfTwoSided,
		class FCardRepresentationData& OutData)
	{
		FMeshDataForDerivedDataTask MeshData;
		MeshData.SourceMeshData = &SourceMeshData;
		MeshData.LODModel = &LODModel;
		MeshData.SectionData = SectionData;
		MeshData.Bounds = (FBoxSphereBounds3f)Bounds;

		return GenerateCardRepresentationData(MeshName, MeshData, DistanceFieldVolumeData, MaxLumenMeshCards, bGenerateAsIfTwoSided, OutData);
	}

	/** Helper structure for skeletal mesh import options */
	struct MeshBuildOptions
	{
		MeshBuildOptions()
		: BoneInfluenceLimit(0)
		, bRemoveDegenerateTriangles(false)
		, bComputeNormals(true)
		, bComputeTangents(true)
		, bUseMikkTSpace(false)
		, bComputeWeightedNormals(false)
		, bCacheOptimize(true)
		, TargetPlatform(nullptr)
		{
		}

		int32 BoneInfluenceLimit;
		bool bRemoveDegenerateTriangles;
		bool bComputeNormals;
		bool bComputeTangents;
		bool bUseMikkTSpace;
		bool bComputeWeightedNormals;
		bool bCacheOptimize;
		FOverlappingThresholds OverlappingThresholds;
		const class ITargetPlatform* TargetPlatform;

		void FillOptions(const FSkeletalMeshBuildSettings& SkeletalMeshBuildSettings)
		{
			OverlappingThresholds.ThresholdPosition = SkeletalMeshBuildSettings.ThresholdPosition;
			OverlappingThresholds.ThresholdTangentNormal = SkeletalMeshBuildSettings.ThresholdTangentNormal;
			OverlappingThresholds.ThresholdUV = SkeletalMeshBuildSettings.ThresholdUV;
			OverlappingThresholds.MorphThresholdPosition = SkeletalMeshBuildSettings.MorphThresholdPosition;
			BoneInfluenceLimit = SkeletalMeshBuildSettings.BoneInfluenceLimit;
			bComputeNormals = SkeletalMeshBuildSettings.bRecomputeNormals;
			bComputeTangents = SkeletalMeshBuildSettings.bRecomputeTangents;
			bUseMikkTSpace = SkeletalMeshBuildSettings.bUseMikkTSpace;
			bComputeWeightedNormals = SkeletalMeshBuildSettings.bComputeWeightedNormals;
			bRemoveDegenerateTriangles = SkeletalMeshBuildSettings.bRemoveDegenerates;
		}
	};
	
	/**
	 * Create all render specific data for a skeletal mesh LOD model
	 * @returns true if the mesh was built successfully.
	 */
	virtual bool BuildSkeletalMesh( 
		FSkeletalMeshLODModel& LODModel,
		const FString& SkeletalMeshName,
		const FReferenceSkeleton& RefSkeleton,
		const TArray<SkeletalMeshImportData::FVertInfluence>& Influences,
		const TArray<SkeletalMeshImportData::FMeshWedge>& Wedges,
		const TArray<SkeletalMeshImportData::FMeshFace>& Faces,
		const TArray<FVector3f>& Points,
		const TArray<int32>& PointToOriginalMap,
		const MeshBuildOptions& BuildOptions = MeshBuildOptions(),
		TArray<FText> * OutWarningMessages = NULL,
		TArray<FName> * OutWarningNames = NULL
		) = 0;

	
	/** Cache optimize the index buffer. */
	virtual void CacheOptimizeIndexBuffer(TArray<uint16>& Indices) = 0;

	/** Cache optimize the index buffer. */
	virtual void CacheOptimizeIndexBuffer(TArray<uint32>& Indices) = 0;

	/** Build adjacency information for the skeletal mesh used for tessellation. */
	virtual void BuildSkeletalAdjacencyIndexBuffer(
		const TArray<struct FSoftSkinVertex>& VertexBuffer,
		const uint32 TexCoordCount,
		const TArray<uint32>& Indices,
		TArray<uint32>& OutPnAenIndices
		) = 0;

	/**
	 *  Calculate The tangent, bi normal and normal for the triangle define by the tree SoftSkinVertex.
	 *
	 *  @note The function will always fill properly the OutTangents array with 3 FVector. If the triangle is degenerated the OutTangent will contain zeroed vectors.
	 *
	 *  @param VertexA - First triangle vertex.
	 *  @param VertexB - Second triangle vertex.
	 *  @param VertexC - Third triangle vertex.
	 *  @param OutTangents - The function allocate the TArray with 3 FVector, to represent the triangle tangent, bi normal and normal.
	 *  @param CompareThreshold - The threshold use to compare a tangent vector with zero.
	 */
	virtual void CalculateTriangleTangent(const FSoftSkinVertex& VertexA, const FSoftSkinVertex& VertexB, const FSoftSkinVertex& VertexC, TArray<FVector3f>& OutTangents, float CompareThreshold) = 0;

	/**
	 *	Calculate the verts associated weighted to each bone of the skeleton.
	 *	The vertices returned are in the local space of the bone.
	 *
	 *	@param	SkeletalMesh	The target skeletal mesh.
	 *	@param	Infos			The output array of vertices associated with each bone.
	 *	@param	bOnlyDominant	Controls whether a vertex is added to the info for a bone if it is most controlled by that bone, or if that bone has ANY influence on that vert.
	 */
	virtual void CalcBoneVertInfos( USkeletalMesh* SkeletalMesh, TArray<FBoneVertInfo>& Infos, bool bOnlyDominant, int32 LodIndex = 0) = 0;

	/**
	 * Convert a set of mesh components in their current pose to a static mesh. 
	 * @param	InMeshComponents		The mesh components we want to convert
	 * @param	InRootTransform			The transform of the root of the mesh we want to output
	 * @param	InPackageName			The package name to create the static mesh in. If this is empty then a dialog will be displayed to pick the mesh.
	 * @return a new static mesh (specified by the user)
	 */
	virtual UStaticMesh* ConvertMeshesToStaticMesh(const TArray<UMeshComponent*>& InMeshComponents, const FTransform& InRootTransform = FTransform::Identity, const FString& InPackageName = FString()) = 0;

	/**
	*	Calculates UV coordinates bounds for the given Skeletal Mesh
	*
	* @param InRawMesh - Skeletal Mesh to calculate the bounds for
	* @param OutBounds - Out texture bounds (min-max)
	*/
	virtual void CalculateTextureCoordinateBoundsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, TArray<FBox2D>& OutBounds) const = 0;
	
	/** Calculates (new) non-overlapping UV coordinates for the given Skeletal Mesh
	*
	* @param LODModel - Skeletal Mesh to generate UV coordinates for
	* @param TextureResolution - Texture resolution to take into account while generating the UVs
	* @param OutTexCoords - New set of UV coordinates
	* @return bool - whether or not generating the UVs succeeded
	*/
	virtual bool GenerateUniqueUVsForSkeletalMesh(const FSkeletalMeshLODModel& LODModel, int32 TextureResolution, TArray<FVector2f>& OutTexCoords) const = 0;
	
	/**
	 * Remove Bones based on LODInfo setting
	 *
	 * @param SkeletalMesh	Mesh that needs bones to be removed
	 * @param LODIndex		Desired LOD to remove bones [ 0 based ]
	 * @param BoneNamesToRemove	List of bone names to remove
	 *
	 * @return true if success
	 */
	virtual bool RemoveBonesFromMesh(USkeletalMesh* SkeletalMesh, int32 LODIndex, const TArray<FName>* BoneNamesToRemove) const = 0;

	/**
	 * Predict which bones bone reduction will remove for a given LOD and compute the reparent map.
	 * This is a read-only operation — it does not modify the mesh. Used by the unified bone map
	 * builder to determine reparent targets that need unified bone slots.
	 *
	 * @param SkeletalMesh		Mesh to inspect
	 * @param LODIndex			LOD to query
	 * @param OutBonesToReplace	Map of [removed bone index -> reparent target bone index]
	 *
	 * @return true if any bones would be removed
	 */
	virtual bool GetBoneReductionData(const USkeletalMesh* SkeletalMesh, int32 LODIndex, TMap<uint16, uint16>& OutBonesToReplace) const = 0;

	/** 
	 * Calculates Tangents and Normals for a given set of vertex data
	 * 
	 * @param InVertices Vertices that make up the mesh
	 * @param InIndices Indices for the Vertex array
	 * @param InUVs Texture coordinates (per-index based)
	 * @param InSmoothingGroupIndices Smoothing group index (per-face based)
	 * @param InTangentOptions Flags for Tangent calculation
	 * @param OutTangentX Array to hold calculated Tangents
	 * @param OutTangentY Array to hold calculated Bitangents
	 * @param OutNormals Array to hold calculated normals (if already contains normals will use those instead for the tangent calculation)	
	 */
	virtual void CalculateTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY, TArray<FVector3f>& OutNormals) const = 0;

	/**
	 * Calculates MikkTSpace Tangents for a given set of vertex data with normals provided
	 *
	 * @param InVertices Vertices that make up the mesh
	 * @param InIndices Indices for the Vertex array
	 * @param InUVs Texture coordinates (per-index based)
	 * @param InNormals Normals used for the tangent calculation (must be normalized)
	 * @param bIgnoreDegenerateTriangles Flag for MikkTSpace to skip degenerate triangles fix-up path
	 * @param OutTangentX Array to hold calculated Tangents
	 * @param OutTangentY Array to hold calculated Bitangents
	 */
	virtual void CalculateMikkTSpaceTangents(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<FVector3f>& InNormals, bool bIgnoreDegenerateTriangles, TArray<FVector3f>& OutTangentX, TArray<FVector3f>& OutTangentY) const = 0;

	/** 
	 * Calculates Normals for a given set of vertex data
	 * 
	 * @param InVertices Vertices that make up the mesh
	 * @param InIndices Indices for the Vertex array
	 * @param InUVs Texture coordinates (per-index based)
	 * @param InSmoothingGroupIndices Smoothing group index (per-face based)
	 * @param InTangentOptions Flags for Tangent calculation
	 * @param OutNormals Array to hold calculated normals	
	 */
	virtual void CalculateNormals(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, const TArray<FVector2f>& InUVs, const TArray<uint32>& InSmoothingGroupIndices, const uint32 InTangentOptions, TArray<FVector3f>& OutNormals) const = 0;

	/** 
	 * Calculates the overlapping corners for a given set of vertex data
	 * 
	 * @param InVertices Vertices that make up the mesh
	 * @param InIndices Indices for the Vertex array
	 * @param bIgnoreDegenerateTriangles Indicates if we should skip degenerate triangles
	 * @param OutOverlappingCorners Container to hold the overlapping corners. For a vertex, lists all the overlapping vertices.
	 */
	virtual void CalculateOverlappingCorners(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, bool bIgnoreDegenerateTriangles, FOverlappingCorners& OutOverlappingCorners) const = 0;

	virtual void RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, FRawMesh &OutRawMesh) const = 0;
	virtual void RecomputeTangentsAndNormalsForRawMesh(bool bRecomputeTangents, bool bRecomputeNormals, const FMeshBuildSettings& InBuildSettings, const FOverlappingCorners& InOverlappingCorners, FRawMesh &OutRawMesh) const = 0;

	virtual void FindOverlappingCorners(FOverlappingCorners& OutOverlappingCorners, const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, float ComparisonThreshold) const = 0;

	/** Used to generate runtime skin weight data from Editor-only data */
	virtual void GenerateRuntimeSkinWeightData(
		const FSkeletalMeshLODModel* ImportedModel,
		const TArray<FRawSkinWeight>& InRawSkinWeights,
		bool bInUseHighPrecisionWeights,
		struct FRuntimeSkinWeightProfileData& InOutSkinWeightOverrideData,
		const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices = nullptr) const = 0;

	/*
	 * This function create the import data using the LODModel. You can call this function if you load an asset that was not re-import since the build refactor and the chunking is more agressive than the bake data in the LODModel.
	 * You can also need this function if you create a skeletalmesh with LODModel instead of import data, so your newly created skeletalmesh can be build properly.
	 * If the LODModel is not being pulled out of the old reduction storage, and bInResetReductionAsNeeded is true, then the reduction settings for that
	 * LOD will be reset to avoid regenerating the mesh even more reduced.
	 */
	virtual void CreateImportDataFromLODModel(USkeletalMesh* InSkeletalMesh, bool bInResetReductionAsNeeded = false) const = 0;
};
